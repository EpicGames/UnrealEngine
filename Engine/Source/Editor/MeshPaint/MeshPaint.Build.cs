// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class MeshPaint : ModuleRules
{
    public MeshPaint(ReadOnlyTargetRules Target) : base(Target)
    {
		PrivateIncludePathModuleNames.AddRange(
            new string[] {
                "AssetRegistry",
                "AssetTools"
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "AppFramework",
                "Core", 
                "CoreUObject",
                "DesktopPlatform",
                "Engine", 
                "InputCore",
                "RenderCore",
                "RHI",
                "Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
                "MeshDescription",
				"StaticMeshDescription",
                "SourceControl",
                "PropertyEditor",
                "MainFrame",
            }
        );

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"AssetTools",
				"LevelEditor"
            });

		DynamicallyLoadedModuleNames.AddRange(
            new string[] {
                "AssetRegistry",
                "AssetTools"
            }
        );
    }
}
