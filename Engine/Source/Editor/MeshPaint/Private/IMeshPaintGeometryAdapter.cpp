// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMeshPaintGeometryAdapter.h"

#include "Engine/World.h"
#include "Components/MeshComponent.h"
#include "MeshPaintTypes.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "MaterialShared.h"
#include "TexturePaintHelpers.h"

//////////////////////////////////////////////////////////////////////////
// IMeshPaintGeometryAdapter

void IMeshPaintGeometryAdapter::DefaultApplyOrRemoveTextureOverride(UMeshComponent* InMeshComponent, UTexture* SourceTexture, UTexture* OverrideTexture)
{
	const ERHIFeatureLevel::Type FeatureLevel = InMeshComponent->GetWorld()->GetFeatureLevel();

	// Check all the materials on the mesh to see if the user texture is there
	int32 MaterialIndex = 0;
	UMaterialInterface* MaterialToCheck = InMeshComponent->GetMaterial(MaterialIndex);
	while (MaterialToCheck != nullptr)
	{
		const bool bIsTextureUsed = DoesMaterialUseTexture(MaterialToCheck, SourceTexture);
		if (bIsTextureUsed)
		{
			MaterialToCheck->OverrideTexture(SourceTexture, OverrideTexture, FeatureLevel);
		}

		++MaterialIndex;
		MaterialToCheck = InMeshComponent->GetMaterial(MaterialIndex);
	}
}

void IMeshPaintGeometryAdapter::DefaultQueryPaintableTextures(int32 MaterialIndex, const UMeshComponent* MeshComponent, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{
	OutDefaultIndex = INDEX_NONE;

	// We already know the material we are painting on, take it off the static mesh component
	UMaterialInterface* Material = MeshComponent->GetMaterial(MaterialIndex);

	if (Material != NULL)
	{
		FPaintableTexture PaintableTexture;
		// Find all the unique textures used in the top material level of the selected actor materials

		// Only grab the textures from the top level of samples
		for (UMaterialExpression* Expression : Material->GetMaterial()->GetExpressions())
		{
			UMaterialExpressionTextureBase* TextureBase = Cast<UMaterialExpressionTextureBase>(Expression);
			if (TextureBase != NULL &&
				TextureBase->Texture != NULL &&
				!TextureBase->Texture->IsNormalMap() && 
				!TextureBase->Texture->VirtualTextureStreaming &&
				!TextureBase->Texture->HasHDRSource() && // Currently HDR textures are not supported to paint on.
				(TextureBase->Texture->Source.GetBytesPerPixel() <= TexturePaintHelpers::GetMaxSupportedBytesPerPixelForPainting())) // Textures' sources must fit in FColor struct to be supported.
			{
				// Default UV channel to index 0. 
				PaintableTexture = FPaintableTexture(TextureBase->Texture, 0);

				// Texture Samples can have UV's specified, check the first node for whether it has a custom UV channel set. 
				// We only check the first as the Mesh paint mode does not support painting with UV's modified in the shader.
				UMaterialExpressionTextureSample* TextureSample = Cast<UMaterialExpressionTextureSample>(Expression);
				if (TextureSample != NULL)
				{
					UMaterialExpressionTextureCoordinate* TextureCoords = Cast<UMaterialExpressionTextureCoordinate>(TextureSample->Coordinates.Expression);
					if (TextureCoords != NULL)
					{
						// Store the uv channel, this is set when the texture is selected. 
						PaintableTexture.UVChannelIndex = TextureCoords->CoordinateIndex;
					}

					// Handle texture parameter expressions
					UMaterialExpressionTextureSampleParameter* TextureSampleParameter = Cast<UMaterialExpressionTextureSampleParameter>(TextureSample);
					if (TextureSampleParameter != NULL)
					{
						// Grab the overridden texture if it exists.  
						Material->GetTextureParameterValue(TextureSampleParameter->ParameterName, PaintableTexture.Texture);
					}
				}

				// note that the same texture will be added again if its UV channel differs. 
				int32 TextureIndex = InOutTextureList.AddUnique(PaintableTexture);

				// cache the first default index, if there is no previous info this will be used as the selected texture
				if ((OutDefaultIndex == INDEX_NONE) && TextureBase->IsDefaultMeshpaintTexture)
				{
					OutDefaultIndex = TextureIndex;
				}
			}
		}
	}
}