// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMeshPaintMode.h"

#include "SceneView.h"
#include "EditorViewportClient.h"
#include "Modules/ModuleManager.h"
#include "EditorReimportHandler.h"
#include "EngineUtils.h"
#include "Utils.h"
#include "UnrealEdGlobals.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "Toolkits/ToolkitManager.h"

#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorSupportDelegates.h"

#include "MeshPaintHelpers.h"
#include "UObject/ObjectSaveContext.h"

//Slate dependencies
#include "LevelEditor.h"
#include "IAssetViewport.h"

#include "MeshPaintAdapterFactory.h"

#include "Components/PrimitiveComponent.h"

#include "IMeshPainter.h"
#include "MeshPaintSettings.h"
#include "AssetRegistry/AssetData.h"

#define LOCTEXT_NAMESPACE "IMeshPaint_Mode"

DEFINE_LOG_CATEGORY_STATIC(LogMeshPaintEdMode, Log, All);

/** Constructor */
IMeshPaintEdMode::IMeshPaintEdMode() 
	: FEdMode()
{
	GEditor->OnEditorClose().AddRaw(this, &IMeshPaintEdMode::OnResetViewMode);
}

IMeshPaintEdMode::~IMeshPaintEdMode()
{
	if (GEditor)
	{
		GEditor->OnEditorClose().RemoveAll(this);
	}
}

/** FGCObject interface */
void IMeshPaintEdMode::AddReferencedObjects( FReferenceCollector& Collector )
{
	// Call parent implementation
	FEdMode::AddReferencedObjects(Collector);
	MeshPainter->AddReferencedObjects(Collector);
}

void IMeshPaintEdMode::Enter()
{
	// Call parent implementation
	FEdMode::Enter();

	checkf(MeshPainter != nullptr, TEXT("Mesh Paint was not initialized"));

	// The user can manipulate the editor selection lock flag in paint mode so we save off the value here so it can be restored later
	bWasSelectionLockedOnStart = GEdSelectionLock;	

	// Catch when objects are replaced when a construction script is rerun
	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &IMeshPaintEdMode::OnObjectsReplaced);

	// Hook into pre/post world save, so that the original collision volumes can be temporarily reinstated
	FEditorDelegates::PreSaveWorldWithContext.AddSP(this, &IMeshPaintEdMode::OnPreSaveWorld);
	FEditorDelegates::PostSaveWorldWithContext.AddSP(this, &IMeshPaintEdMode::OnPostSaveWorld);

	// Catch assets if they are about to be (re)imported
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddSP(this, &IMeshPaintEdMode::OnPostImportAsset);
	FReimportManager::Instance()->OnPostReimport().AddSP(this, &IMeshPaintEdMode::OnPostReimportAsset);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &IMeshPaintEdMode::OnAssetRemoved);

	// Initialize adapter globals
	FMeshPaintAdapterFactory::InitializeAdapterGlobals();

	SelectionChangedHandle = USelection::SelectionChangedEvent.AddLambda([this](UObject* Object) { MeshPainter->Refresh();  });

	if (UsesToolkits() && !Toolkit.IsValid())
	{
		Toolkit = GetToolkit();
		Toolkit->Init(Owner->GetToolkitHost());
	}
	
	// Change the engine to draw selected objects without a color boost, but unselected objects will
	// be darkened slightly.  This just makes it easier to paint on selected objects without the
	// highlight effect distorting the appearance.
	GEngine->OverrideSelectedMaterialColor( FLinearColor::Black );

	if (UsesToolkits())
	{
		MeshPainter->RegisterCommands(Toolkit->GetToolkitCommands());
	}
		
	MeshPainter->Refresh();
}

void IMeshPaintEdMode::Exit()
{
	/** Finish up painting if we still are */
	if (MeshPainter->IsPainting())
	{
		MeshPainter->FinishPainting();
	}

	/** Reset paint state and unregister commands */
	MeshPainter->Reset();	
	if (UsesToolkits())
	{
		MeshPainter->UnregisterCommands(Toolkit->GetToolkitCommands());
	}

	// The user can manipulate the editor selection lock flag in paint mode so we make sure to restore it here
	GEdSelectionLock = bWasSelectionLockedOnStart;
	
	OnResetViewMode();

	// Restore selection color
	GEngine->RestoreSelectedMaterialColor();

	if (Toolkit.IsValid())
	{
		FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
		Toolkit.Reset();
	}

	// Unbind delegates
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRemoved().RemoveAll(this);
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
	FEditorDelegates::PreSaveWorldWithContext.RemoveAll(this);
	FEditorDelegates::PostSaveWorldWithContext.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	USelection::SelectionChangedEvent.Remove(SelectionChangedHandle);

	// Call parent implementation
	FEdMode::Exit();
}

bool IMeshPaintEdMode::ProcessCapturedMouseMoves( FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves)
{
	// We only care about perspective viewpo1rts
	bool bPaintApplied = false;
	if( InViewportClient->IsPerspective() )
	{
		if( MeshPainter->IsPainting() && CapturedMouseMoves.Num() > 0 )
		{
			// Compute a world space ray from the screen space mouse coordinates
			FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( 
				InViewportClient->Viewport, 
				InViewportClient->GetScene(),
				InViewportClient->EngineShowFlags)
				.SetRealtimeUpdate( InViewportClient->IsRealtime() ));
			FSceneView* View = InViewportClient->CalcSceneView( &ViewFamily );

			TArray<TPair<FVector, FVector>> Rays;
			Rays.Reserve(CapturedMouseMoves.Num());

			FEditorViewportClient* Client = (FEditorViewportClient*)InViewport->GetClient();
			for (int32 i = 0; i < CapturedMouseMoves.Num(); ++i)
			{
				FViewportCursorLocation MouseViewportRay(View, Client, CapturedMouseMoves[i].X, CapturedMouseMoves[i].Y);
				Rays.Emplace(TPair<FVector, FVector>(MouseViewportRay.GetOrigin(), MouseViewportRay.GetDirection()));
			}
			 					
			bPaintApplied = MeshPainter->Paint(InViewport, View->ViewMatrices.GetViewOrigin(), Rays);
		}
	}

	return bPaintApplied;
}

/** FEdMode: Called when a mouse button is released */
bool IMeshPaintEdMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	InViewportClient->bLockFlightCamera = false;
	if (MeshPainter->IsPainting())
	{
		MeshPainter->FinishPainting();
	}

	return true;
}

/** FEdMode: Called when a key is pressed */
bool IMeshPaintEdMode::InputKey( FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent )
{
	bool bHandled = MeshPainter->InputKey(InViewportClient, InViewport, InKey, InEvent);

	if (bHandled)
	{
		return bHandled;
	}
	const bool bIsLeftButtonDown = ( InKey == EKeys::LeftMouseButton && InEvent != IE_Released ) || InViewport->KeyState( EKeys::LeftMouseButton );
	const bool bIsRightButtonDown = (InKey == EKeys::RightMouseButton && InEvent != IE_Released) || InViewport->KeyState(EKeys::RightMouseButton);
	const bool bIsCtrlDown = ((InKey == EKeys::LeftControl || InKey == EKeys::RightControl) && InEvent != IE_Released) || InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	const bool bIsShiftDown = ( ( InKey == EKeys::LeftShift || InKey == EKeys::RightShift ) && InEvent != IE_Released ) || InViewport->KeyState( EKeys::LeftShift ) || InViewport->KeyState( EKeys::RightShift );
	const bool bIsAltDown = ( ( InKey == EKeys::LeftAlt || InKey == EKeys::RightAlt ) && InEvent != IE_Released ) || InViewport->KeyState( EKeys::LeftAlt ) || InViewport->KeyState( EKeys::RightAlt );

	// When painting we only care about perspective viewports
	if( !bIsAltDown && InViewportClient->IsPerspective() )
	{
		// Does the user want to paint right now?
		const bool bUserWantsPaint = bIsLeftButtonDown && !bIsRightButtonDown && !bIsAltDown;
		bool bPaintApplied = false;

		// Stop current tracking if the user is no longer painting
		if( MeshPainter->IsPainting() && !bUserWantsPaint &&
			( InKey == EKeys::LeftMouseButton || InKey == EKeys::RightMouseButton || InKey == EKeys::LeftAlt || InKey == EKeys::RightAlt ) )
		{
			bHandled = true;
			MeshPainter->FinishPainting();
			InViewportClient->bLockFlightCamera = false;
		}
		else if( !MeshPainter->IsPainting() && bUserWantsPaint && !InViewportClient->IsMovingCamera())
		{	
			bHandled = true;

			// Compute a world space ray from the screen space mouse coordinates
			FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( 
				InViewportClient->Viewport, 
				InViewportClient->GetScene(),
				InViewportClient->EngineShowFlags )
				.SetRealtimeUpdate( InViewportClient->IsRealtime() ));

			FSceneView* View = InViewportClient->CalcSceneView( &ViewFamily );
			const FViewportCursorLocation MouseViewportRay( View, (FEditorViewportClient*)InViewport->GetClient(), InViewport->GetMouseX(), InViewport->GetMouseY() );

			// Paint!
			bPaintApplied = MeshPainter->Paint(InViewport, View->ViewMatrices.GetViewOrigin(), MouseViewportRay.GetOrigin(), MouseViewportRay.GetDirection());
			
		}
		else if (MeshPainter->IsPainting() && bUserWantsPaint)
		{
			bHandled = true;
		}

		if( !bPaintApplied && !MeshPainter->IsPainting())
		{
			bHandled = false;
		}
		else
		{
			InViewportClient->bLockFlightCamera = true;
		}

		// Also absorb other mouse buttons, and Ctrl/Alt/Shift events that occur while we're painting as these would cause
		// the editor viewport to start panning/dollying the camera
		{
			const bool bIsOtherMouseButtonEvent = ( InKey == EKeys::MiddleMouseButton || InKey == EKeys::RightMouseButton );
			const bool bCtrlButtonEvent = (InKey == EKeys::LeftControl || InKey == EKeys::RightControl);
			const bool bShiftButtonEvent = (InKey == EKeys::LeftShift || InKey == EKeys::RightShift);
			const bool bAltButtonEvent = (InKey == EKeys::LeftAlt || InKey == EKeys::RightAlt);
			if( MeshPainter->IsPainting() && ( bIsOtherMouseButtonEvent || bShiftButtonEvent || bAltButtonEvent ) )
			{
				bHandled = true;
			}

			if( bCtrlButtonEvent && !MeshPainter->IsPainting())
			{
				bHandled = false;
			}
			else if( bIsCtrlDown)
			{
				//default to assuming this is a paint command
				bHandled = true;

				// Allow Ctrl+B to pass through so we can support the finding of a selected static mesh in the content browser.
				if ( !(bShiftButtonEvent || bAltButtonEvent || bIsOtherMouseButtonEvent) && ( (InKey == EKeys::B) && (InEvent == IE_Pressed) ) )
				{
					bHandled = false;
				}

				// If we are not painting, we will let the CTRL-Z and CTRL-Y key presses through to support undo/redo.
				if ( !MeshPainter->IsPainting() && ( InKey == EKeys::Z || InKey == EKeys::Y ) )
				{
					bHandled = false;
				}
			}
		}
	}

	return bHandled;
}

void IMeshPaintEdMode::OnPreSaveWorld(UWorld* World, FObjectPreSaveContext ObjectSaveContext)
{
	MeshPainter->Refresh();
}

void IMeshPaintEdMode::OnPostSaveWorld(UWorld* World, FObjectPostSaveContext ObjectSaveContext)
{
	MeshPainter->Refresh();
}

void IMeshPaintEdMode::OnPostImportAsset(UFactory* Factory, UObject* Object)
{
	MeshPainter->Refresh();
}

void IMeshPaintEdMode::OnPostReimportAsset(UObject* Object, bool bSuccess)
{
	MeshPainter->Refresh();
}

void IMeshPaintEdMode::OnAssetRemoved(const FAssetData& AssetData)
{
	MeshPainter->Refresh();
}

void IMeshPaintEdMode::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	MeshPainter->Refresh();
}

void IMeshPaintEdMode::OnResetViewMode()
{
	// Reset viewport color mode for all active viewports
	for(FEditorViewportClient* ViewportClient : GEditor->GetAllViewportClients())
	{
		if (!ViewportClient || ViewportClient->GetModeTools() != GetModeManager())
		{
			continue;
		}

		MeshPaintHelpers::SetViewportColorMode(EMeshPaintColorViewMode::Normal, ViewportClient);
	}
}

/** FEdMode: Called after an Undo operation */
void IMeshPaintEdMode::PostUndo()
{
	FEdMode::PostUndo();
	MeshPainter->Refresh();
}

/** FEdMode: Render the mesh paint tool */
void IMeshPaintEdMode::Render( const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI )
{
	/** Call parent implementation */
	FEdMode::Render( View, Viewport, PDI );
	MeshPainter->Render(View, Viewport, PDI);

	// Flow painting
	if (MeshPainter->IsPainting() && MeshPainter->GetBrushSettings()->bEnableFlow)
	{
		// Make sure the cursor is visible OR we're flood filling.  No point drawing a paint cue when there's no cursor.
		if (Viewport->IsCursorVisible())
		{				
			// Grab the mouse cursor position
			FIntPoint MousePosition;
			Viewport->GetMousePos(MousePosition);

			// Is the mouse currently over the viewport? or flood filling
			if ((MousePosition.X >= 0 && MousePosition.Y >= 0 && MousePosition.X < (int32)Viewport->GetSizeXY().X && MousePosition.Y < (int32)Viewport->GetSizeXY().Y))
			{
				// Compute a world space ray from the screen space mouse coordinates
				FViewportCursorLocation MouseViewportRay(View, static_cast<FEditorViewportClient*>(Viewport->GetClient()), MousePosition.X, MousePosition.Y);
				MeshPainter->Paint(Viewport, View->ViewMatrices.GetViewOrigin(), MouseViewportRay.GetOrigin(), MouseViewportRay.GetDirection());
			}
		}
	}
}

/** FEdMode: Handling SelectActor */
bool IMeshPaintEdMode::Select( AActor* InActor, bool bInSelected )
{
	if (bInSelected)
	{
		MeshPainter->ActorSelected(InActor);
	}
	else
	{
		MeshPainter->ActorDeselected(InActor);
	}
	
	return false;
}

/** FEdMode: Called when the currently selected actor has changed */
void IMeshPaintEdMode::ActorSelectionChangeNotify()
{
	MeshPainter->Refresh();
}

/** IMeshPaintEdMode: Called once per frame */
void IMeshPaintEdMode::Tick(FEditorViewportClient* ViewportClient,float DeltaTime)
{
	FEdMode::Tick(ViewportClient,DeltaTime);
	MeshPainter->Tick(ViewportClient, DeltaTime);
}

IMeshPainter* IMeshPaintEdMode::GetMeshPainter()
{
	checkf(MeshPainter != nullptr, TEXT("Invalid Mesh painter ptr"));
	return MeshPainter;
}

bool IMeshPaintEdMode::ProcessEditDelete()
{
	MeshPainter->Refresh();
	return false;
}

#undef LOCTEXT_NAMESPACE
