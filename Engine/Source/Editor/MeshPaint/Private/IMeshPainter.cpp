// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMeshPainter.h"

#include "Editor.h"
#include "IMeshPaintGeometryAdapter.h"
#include "MeshPaintSettings.h"
#include "MeshPainterCommands.h"
#include "MeshPaintHelpers.h"
#include "ScopedTransaction.h"

#include "Components/MeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "EditorViewportClient.h"

IMeshPainter::IMeshPainter() :
	bArePainting(false),
	TimeSinceStartedPainting(0.0f),
	Time(0.0f),
	WidgetLineThickness(1.0f),
	VertexPointColor(FLinearColor::White),
	HoverVertexPointColor(0.3f, 1.0f, 0.3f),
	PaintTransaction(nullptr)
{
}

IMeshPainter::~IMeshPainter()
{
	UICommandList.Reset();
}

void IMeshPainter::RenderInteractors(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI, bool bRenderVertices, ESceneDepthPriorityGroup DepthGroup/* = SDPG_World*/)
{
	TArray<MeshPaintHelpers::FPaintRay> PaintRays;
	MeshPaintHelpers::RetrieveViewportPaintRays(View, Viewport, PDI, PaintRays);

	// Apply paint pressure and start painting (or if not currently painting, draw a preview of where paint will be applied)
	for (const MeshPaintHelpers::FPaintRay& PaintRay : PaintRays)
	{
		EMeshPaintAction RayPaintAction = (Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl)) ? EMeshPaintAction::Erase : EMeshPaintAction::Paint;

		RenderInteractorWidget(PaintRay.CameraLocation, PaintRay.RayStart, PaintRay.RayDirection, PDI, RayPaintAction, bRenderVertices, DepthGroup);
	}
}

void IMeshPainter::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	if (bArePainting)
	{
		TimeSinceStartedPainting += DeltaTime;
	}
	
	if (ViewportClient->IsPerspective())
	{
		// Make sure perspective viewports are still set to real-time
		MeshPaintHelpers::SetRealtimeViewport(true);

		// Set viewport show flags		
		MeshPaintHelpers::SetViewportColorMode(GetBrushSettings()->ColorViewMode, ViewportClient);
	}

	Time += DeltaTime;
}

void IMeshPainter::ChangeBrushRadius(float Multiplier)
{
	const float ChangePercentage = 0.05f;
	const float OldValue = GetBrushSettings()->GetBrushRadius();

	float NewValue = OldValue * (1 + ChangePercentage * Multiplier);
	GetBrushSettings()->SetBrushRadius(NewValue);
}

void IMeshPainter::ChangeBrushStrength(float Multiplier)
{
	const float ChangeAmount = 0.02f;
	const float OldValue = GetBrushSettings()->GetBrushStrength();

	float NewValue = OldValue + ChangeAmount * Multiplier;
	GetBrushSettings()->SetBrushStrength(NewValue);
}

void IMeshPainter::ChangeBrushFalloff(float Multiplier)
{
	const float ChangeAmount = 0.02f;
	const float OldValue = GetBrushSettings()->GetBrushFalloff();

	float NewValue = OldValue + ChangeAmount * Multiplier;
	GetBrushSettings()->SetBrushFalloff(NewValue);
}

void IMeshPainter::RegisterCommands(TSharedRef<FUICommandList> CommandList)
{
	const FMeshPainterCommands& Commands = FMeshPainterCommands::Get();

	CommandList->MapAction(Commands.IncreaseBrushRadius, FExecuteAction::CreateRaw(this, &IMeshPainter::ChangeBrushRadius, 1.0f), FCanExecuteAction(), EUIActionRepeatMode::RepeatEnabled);
	CommandList->MapAction(Commands.DecreaseBrushRadius, FExecuteAction::CreateRaw(this, &IMeshPainter::ChangeBrushRadius, -1.0f), FCanExecuteAction(), EUIActionRepeatMode::RepeatEnabled);

	CommandList->MapAction(Commands.IncreaseBrushStrength, FExecuteAction::CreateRaw(this, &IMeshPainter::ChangeBrushStrength, 1.0f), FCanExecuteAction(), EUIActionRepeatMode::RepeatEnabled);
	CommandList->MapAction(Commands.DecreaseBrushStrength, FExecuteAction::CreateRaw(this, &IMeshPainter::ChangeBrushStrength, -1.0f), FCanExecuteAction(), EUIActionRepeatMode::RepeatEnabled);

	CommandList->MapAction(Commands.IncreaseBrushFalloff, FExecuteAction::CreateRaw(this, &IMeshPainter::ChangeBrushFalloff, 1.0f), FCanExecuteAction(), EUIActionRepeatMode::RepeatEnabled);
	CommandList->MapAction(Commands.DecreaseBrushFalloff, FExecuteAction::CreateRaw(this, &IMeshPainter::ChangeBrushFalloff, -1.0f), FCanExecuteAction(), EUIActionRepeatMode::RepeatEnabled);

	UICommandList = CommandList;
}

void IMeshPainter::UnregisterCommands(TSharedRef<FUICommandList> CommandList)
{
	UICommandList.Reset();

	const FMeshPainterCommands& Commands = FMeshPainterCommands::Get();	
	for (const TSharedPtr<const FUICommandInfo> Action : Commands.Commands)
	{
		CommandList->UnmapAction(Action);
	}
}

bool IMeshPainter::Paint(FViewport* Viewport, const FVector& InCameraOrigin, const FVector& InRayOrigin, const FVector& InRayDirection)
{
	// Determine paint action according to whether or not shift is held down
	const EMeshPaintAction PaintAction = (Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift)) ? EMeshPaintAction::Erase : EMeshPaintAction::Paint;
	
	const float PaintStrength = Viewport->IsPenActive() ? Viewport->GetTabletPressure() : 1.f;
	// Handle internal painting functionality
	TPair<FVector, FVector> Ray(InRayOrigin, InRayDirection);
	return PaintInternal(InCameraOrigin, MakeArrayView(&Ray, 1), PaintAction, PaintStrength);
}

bool IMeshPainter::Paint(FViewport* Viewport, const FVector& InCameraOrigin, const TArrayView<TPair<FVector, FVector>>& Rays)
{
	// Determine paint action according to whether or not shift is held down
	const EMeshPaintAction PaintAction = (Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift)) ? EMeshPaintAction::Erase : EMeshPaintAction::Paint;

	const float PaintStrength = Viewport->IsPenActive() ? Viewport->GetTabletPressure() : 1.f;
	// Handle internal painting functionality
	return PaintInternal(InCameraOrigin, Rays, PaintAction, PaintStrength);
}

bool IMeshPainter::InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent)
{
	bool bHandled = false;

	if (UICommandList.IsValid())
	{
		bHandled = UICommandList->ProcessCommandBindings(InKey, FSlateApplication::Get().GetModifierKeys(), InEvent == IE_Repeat);
	}
	
	return bHandled;
}

void IMeshPainter::FinishPainting()
{
	if (bArePainting)
	{
		bArePainting = false;
		if (PaintTransaction)
		{
			EndTransaction();
		}
	}
}

void IMeshPainter::RenderInteractorWidget(const FVector& InCameraOrigin, const FVector& InRayOrigin, const FVector& InRayDirection, FPrimitiveDrawInterface* PDI, EMeshPaintAction PaintAction, bool bRenderVertices, ESceneDepthPriorityGroup DepthGroup /*= SDPG_World*/)
{
	const UPaintBrushSettings* BrushSettings = GetBrushSettings();
	const float BrushRadius = BrushSettings->GetBrushRadius();

	const FHitResult& HitResult = GetHitResult(InRayOrigin, InRayDirection);
	
	const float DrawThickness = WidgetLineThickness;
	const float DrawIntensity = 1.0f;

	if (HitResult.Component != nullptr)
	{
		// Brush properties
		const float BrushFalloffAmount = BrushSettings->BrushFalloffAmount;

		// Display settings
		const float VisualBiasDistance = 0.15f;
		const float NormalLineSize(BrushRadius * 0.35f);	// Make the normal line length a function of brush size
		const FLinearColor NormalLineColor(0.3f, 1.0f, 0.3f);
		const FLinearColor BrushCueColor = (bArePainting ? FLinearColor(1.0f, 1.0f, 0.3f) : FLinearColor(0.3f, 1.0f, 0.3f));
		const FLinearColor InnerBrushCueColor = (bArePainting ? FLinearColor(0.5f, 0.5f, 0.1f) : FLinearColor(0.1f, 0.5f, 0.1f));
		const UMeshPaintSettings* const PaintSettings = GetPainterSettings();
		const float PointDrawSize = PaintSettings ? PaintSettings->VertexPreviewSize : GetDefault<UMeshPaintSettings>()->VertexPreviewSize;

		FVector BrushXAxis, BrushYAxis;
		HitResult.Normal.FindBestAxisVectors(BrushXAxis, BrushYAxis);
		const FVector BrushVisualPosition = HitResult.Location + HitResult.Normal * VisualBiasDistance;

		
		if (PDI != NULL)
		{
			// Draw brush circle
			const int32 NumCircleSides = 128;
			DrawCircle(PDI, BrushVisualPosition, BrushXAxis, BrushYAxis, BrushCueColor, BrushRadius, NumCircleSides, DepthGroup, DrawThickness);

			// Also draw the inner brush radius
			const float InnerBrushRadius = (BrushRadius * BrushFalloffAmount);
			DrawCircle(PDI, BrushVisualPosition, BrushXAxis, BrushYAxis, InnerBrushCueColor, InnerBrushRadius, NumCircleSides, DepthGroup, DrawThickness);
			
			// If we just started painting then also draw a little brush effect
			if (bArePainting)
			{
				const float EffectDuration = 0.2f;

				const double CurTime = FPlatformTime::Seconds();
				if (TimeSinceStartedPainting <= EffectDuration)
				{
					// Invert the effect if we're currently erasing
					float EffectAlpha = TimeSinceStartedPainting / EffectDuration;
					if (PaintAction == EMeshPaintAction::Erase)
					{
						EffectAlpha = 1.0f - EffectAlpha;
					}

					const FLinearColor EffectColor(0.1f + EffectAlpha * 0.4f, 0.1f + EffectAlpha * 0.4f, 0.1f + EffectAlpha * 0.4f);
					const float EffectRadius = BrushRadius * EffectAlpha * EffectAlpha;	// Squared curve here (looks more interesting)
					DrawCircle(PDI, BrushVisualPosition, BrushXAxis, BrushYAxis, EffectColor, EffectRadius, NumCircleSides, DepthGroup);
				}
			}

			// Draw trace surface normal
			const FVector NormalLineEnd(BrushVisualPosition + HitResult.Normal * NormalLineSize);
			PDI->DrawLine(BrushVisualPosition, NormalLineEnd, NormalLineColor, DepthGroup, DrawThickness);

			TSharedPtr<IMeshPaintGeometryAdapter> MeshAdapter = GetMeshAdapterForComponent(Cast<UMeshComponent>(HitResult.Component.Get()));

			if (MeshAdapter->IsValid() && bRenderVertices && MeshAdapter->SupportsVertexPaint())
			{
				const FMatrix ComponentToWorldMatrix = MeshAdapter->GetComponentToWorldMatrix();
				const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(InCameraOrigin));
				const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(HitResult.Location));

				// @todo MeshPaint: Input vector doesn't work well with non-uniform scale
				const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(BrushRadius, 0.0f, 0.0f)).Size();
				const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

				const TArray<FVector>& InRangeVertices = MeshAdapter->SphereIntersectVertices(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, GetBrushSettings()->bOnlyFrontFacingTriangles);

				for (const FVector& Vertex : InRangeVertices)
				{
					const FVector WorldPositionVertex = ComponentToWorldMatrix.TransformPosition(Vertex);	
					if (( HitResult.Location - WorldPositionVertex).Size() <= BrushRadius)
					{
						const FVector VertexVisualPosition = WorldPositionVertex + HitResult.Normal * VisualBiasDistance;
						PDI->DrawPoint(VertexVisualPosition, HoverVertexPointColor, PointDrawSize, DepthGroup);
					}
				}
			}
		}
	}
}

void IMeshPainter::BeginTransaction(const FText Description)
{
	// In paint mode we only allow the BeginTransaction to be called with the EndTransaction pair. We should never be
	// in a state where a second transaction was started before the first was ended.
	checkf(PaintTransaction == NULL, TEXT("Cannot create Transaction while another one is still active"));
	if (PaintTransaction == NULL)
	{
		PaintTransaction = new FScopedTransaction(Description);
	}
}

void IMeshPainter::EndTransaction()
{
	checkf(PaintTransaction != NULL, TEXT("Cannot end Transaction since there isn't one Active"));
	delete PaintTransaction;
	PaintTransaction = NULL;
}
