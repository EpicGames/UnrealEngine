// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintAdapterFactory.h"
#include "IMeshPaintGeometryAdapterFactory.h"

TArray<TSharedPtr<class IMeshPaintGeometryAdapterFactory>> FMeshPaintAdapterFactory::FactoryList;

TSharedPtr<class IMeshPaintGeometryAdapter> FMeshPaintAdapterFactory::CreateAdapterForMesh(UMeshComponent* InComponent, int32 InPaintingMeshLODIndex)
{
	TSharedPtr<IMeshPaintGeometryAdapter> Result;

	for (const auto& Factory : FactoryList)
	{
		Result = Factory->Construct(InComponent, InPaintingMeshLODIndex);

		if (Result.IsValid())
		{
			break;
		}
	}

	return Result;
}

void FMeshPaintAdapterFactory::InitializeAdapterGlobals()
{
	for (const auto& Factory : FactoryList)
	{
		Factory->InitializeAdapterGlobals();
	}
}

void FMeshPaintAdapterFactory::AddReferencedObjectsGlobals(FReferenceCollector& Collector)
{
	for (const auto& Factory : FactoryList)
	{
		Factory->AddReferencedObjectsGlobals(Collector);
	}
}

void FMeshPaintAdapterFactory::CleanupGlobals()
{
	for (const auto& Factory : FactoryList)
	{
		Factory->CleanupGlobals();
	}
}
