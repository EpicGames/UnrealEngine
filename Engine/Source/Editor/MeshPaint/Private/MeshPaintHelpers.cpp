// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintHelpers.h"

#include "ComponentReregisterContext.h"

#include "MeshPaintTypes.h"
#include "MeshPaintSettings.h"
#include "IMeshPaintGeometryAdapter.h"
#include "MeshPaintAdapterFactory.h"
#include "MeshPaintVisualize.h"

#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/Texture2D.h"
#include "SceneView.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "StaticMeshAttributes.h"

#include "Rendering/SkeletalMeshRenderData.h"

#include "Math/GenericOctree.h"
#include "Utils.h"

#include "Framework/Application/SlateApplication.h"
#include "SImportVertexColorOptions.h"
#include "EditorViewportClient.h"
#include "Interfaces/IMainFrameModule.h"

#include "Modules/ModuleManager.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "PackageTools.h"
#include "FileHelpers.h"
#include "ISourceControlModule.h"

#include "Editor.h"
#include "LevelEditor.h"
#include "IAssetViewport.h"
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"

#include "Factories/FbxSkeletalMeshImportData.h"

#include "Async/ParallelFor.h"
#include "Rendering/SkeletalMeshModel.h"

extern void PropagateVertexPaintToAsset(USkeletalMesh* SkeletalMesh, int32 LODIndex);

void MeshPaintHelpers::RemoveInstanceVertexColors(UObject* Obj)
{
	// Currently only static mesh component supports per instance vertex colors so only need to retrieve those and remove colors
	AActor* Actor = Cast<AActor>(Obj);
	if (Actor != nullptr)
	{
		TArray<UStaticMeshComponent*> StaticMeshComponents;
		Actor->GetComponents(StaticMeshComponents);
		for (const auto& StaticMeshComponent : StaticMeshComponents)
		{
			if (StaticMeshComponent != nullptr)
			{
				MeshPaintHelpers::RemoveComponentInstanceVertexColors(StaticMeshComponent);
			}
		}
	}
}

void MeshPaintHelpers::RemoveComponentInstanceVertexColors(UStaticMeshComponent* StaticMeshComponent)
{
	if (StaticMeshComponent != nullptr && StaticMeshComponent->GetStaticMesh() != nullptr)
	{
		// Mark the mesh component as modified
		StaticMeshComponent->Modify();

		// If this is called from the Remove button being clicked the SMC wont be in a Reregister context,
		// but when it gets called from a Paste or Copy to Source operation it's already inside a more specific
		// SMCRecreateScene context so we shouldn't put it inside another one.
		if (StaticMeshComponent->IsRenderStateCreated())
		{
			// Detach all instances of this static mesh from the scene.
			FComponentReregisterContext ComponentReregisterContext(StaticMeshComponent);

			StaticMeshComponent->RemoveInstanceVertexColors();
		}
		else
		{
			StaticMeshComponent->RemoveInstanceVertexColors();
		}
	}
}


bool MeshPaintHelpers::PropagateColorsToRawMesh(UStaticMesh* StaticMesh, int32 LODIndex, FStaticMeshComponentLODInfo& ComponentLODInfo)
{
	check(ComponentLODInfo.OverrideVertexColors);
	check(StaticMesh->IsSourceModelValid(LODIndex));
	check(StaticMesh->GetRenderData());
	check(StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex));

	bool bPropagatedColors = false;
	FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
	FStaticMeshRenderData& RenderData = *StaticMesh->GetRenderData();
	FStaticMeshLODResources& RenderModel = RenderData.LODResources[LODIndex];
	FColorVertexBuffer& ColorVertexBuffer = *ComponentLODInfo.OverrideVertexColors;
	if (RenderModel.WedgeMap.Num() > 0 && ColorVertexBuffer.GetNumVertices() == RenderModel.GetNumVertices())
	{
		// Use the wedge map if it is available as it is lossless.
		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);

		if (MeshDescription == nullptr)
		{
			//Cannot propagate to a generated LOD, the generated LOD use the source LOD vertex painting. 
			return false;
		}

		FStaticMeshAttributes Attributes(*MeshDescription);
		int32 NumWedges = MeshDescription->VertexInstances().Num();
		if (RenderModel.WedgeMap.Num() == NumWedges)
		{
			TVertexInstanceAttributesRef<FVector4f> Colors = Attributes.GetVertexInstanceColors();
			int32 VertexInstanceIndex = 0;
			for (const FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
			{
				FLinearColor WedgeColor = FLinearColor::White;
				int32 Index = RenderModel.WedgeMap[VertexInstanceIndex];
				if (Index != INDEX_NONE)
				{
					WedgeColor = FLinearColor(ColorVertexBuffer.VertexColor(Index));
				}
				Colors[VertexInstanceID] = WedgeColor;
				VertexInstanceIndex++;
			}
			StaticMesh->CommitMeshDescription(LODIndex);
			bPropagatedColors = true;
		}
	}
	else
	{
		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);
		// If there's no raw mesh data, don't try to do any fixup here
		if (MeshDescription == nullptr || ComponentLODInfo.OverrideVertexColors == nullptr)
		{
			return false;
		}

		FStaticMeshAttributes Attributes(*MeshDescription);

		// Fall back to mapping based on position.
		TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TVertexInstanceAttributesRef<FVector4f> Colors = Attributes.GetVertexInstanceColors();
		TArray<FColor> NewVertexColors;
		FPositionVertexBuffer TempPositionVertexBuffer;
		int32 NumVertex = MeshDescription->Vertices().Num();
		TArray<FVector3f> VertexPositionsDup;
		VertexPositionsDup.AddZeroed(NumVertex);
		int32 VertexIndex = 0;
		for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
		{
			VertexPositionsDup[VertexIndex++] = VertexPositions[VertexID];
		}
		TempPositionVertexBuffer.Init(VertexPositionsDup);
		RemapPaintedVertexColors(
			ComponentLODInfo.PaintedVertices,
			ComponentLODInfo.OverrideVertexColors,
			RenderModel.VertexBuffers.PositionVertexBuffer,
			RenderModel.VertexBuffers.StaticMeshVertexBuffer,
			TempPositionVertexBuffer,
			/*OptionalVertexBuffer=*/ nullptr,
			NewVertexColors
		);
		if (NewVertexColors.Num() == NumVertex)
		{
			for (const FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
			{
				const FVertexID VertexID = MeshDescription->GetVertexInstanceVertex(VertexInstanceID);
				Colors[VertexInstanceID] = FLinearColor(NewVertexColors[VertexID.GetValue()]);
			}
			StaticMesh->CommitMeshDescription(LODIndex);
			bPropagatedColors = true;
		}
	}
	return bPropagatedColors;
}

bool MeshPaintHelpers::PaintVertex(const FVector& InVertexPosition, const FMeshPaintParameters& InParams, FColor& InOutVertexColor)
{
	float SquaredDistanceToVertex2D;
	float VertexDepthToBrush;
	if (MeshPaintHelpers::IsPointInfluencedByBrush(InVertexPosition, InParams, SquaredDistanceToVertex2D, VertexDepthToBrush))
	{
		// Compute amount of paint to apply
		const float PaintAmount = ComputePaintMultiplier(SquaredDistanceToVertex2D, InParams.BrushStrength, InParams.InnerBrushRadius, InParams.BrushRadialFalloffRange, InParams.BrushDepth, InParams.BrushDepthFalloffRange, VertexDepthToBrush);
			
		const FLinearColor OldColor = InOutVertexColor.ReinterpretAsLinear();
		FLinearColor NewColor = OldColor;

		if (InParams.PaintMode == EMeshPaintMode::PaintColors)
		{
			ApplyVertexColorPaint(InParams, OldColor, NewColor, PaintAmount);

		}
		else if (InParams.PaintMode == EMeshPaintMode::PaintWeights)
		{
			ApplyVertexWeightPaint(InParams, OldColor, PaintAmount, NewColor);
		}

		// Save the new color
		InOutVertexColor.R = FMath::Clamp(FMath::RoundToInt(NewColor.R * 255.0f), 0, 255);
		InOutVertexColor.G = FMath::Clamp(FMath::RoundToInt(NewColor.G * 255.0f), 0, 255);
		InOutVertexColor.B = FMath::Clamp(FMath::RoundToInt(NewColor.B * 255.0f), 0, 255);
		InOutVertexColor.A = FMath::Clamp(FMath::RoundToInt(NewColor.A * 255.0f), 0, 255);

		return true;
	}

	// Out of range
	return false;
}

void MeshPaintHelpers::ApplyVertexColorPaint(const FMeshPaintParameters &InParams, const FLinearColor &OldColor, FLinearColor &NewColor, const float PaintAmount)
{
	// Color painting
	if (InParams.bWriteRed)
	{
		NewColor.R = (OldColor.R < InParams.BrushColor.R) ? FMath::Min(InParams.BrushColor.R, OldColor.R + PaintAmount) : FMath::Max(InParams.BrushColor.R, OldColor.R - PaintAmount);
	}

	if (InParams.bWriteGreen)
	{
		NewColor.G = (OldColor.G < InParams.BrushColor.G) ? FMath::Min(InParams.BrushColor.G, OldColor.G + PaintAmount) : FMath::Max(InParams.BrushColor.G, OldColor.G - PaintAmount);
	}

	if (InParams.bWriteBlue)
	{
		NewColor.B = (OldColor.B < InParams.BrushColor.B) ? FMath::Min(InParams.BrushColor.B, OldColor.B + PaintAmount) : FMath::Max(InParams.BrushColor.B, OldColor.B - PaintAmount);
	}

	if (InParams.bWriteAlpha)
	{
		NewColor.A = (OldColor.A < InParams.BrushColor.A) ? FMath::Min(InParams.BrushColor.A, OldColor.A + PaintAmount) : FMath::Max(InParams.BrushColor.A, OldColor.A - PaintAmount);
	}
}

void MeshPaintHelpers::ApplyVertexWeightPaint(const FMeshPaintParameters &InParams, const FLinearColor &OldColor, const float PaintAmount, FLinearColor &NewColor)
{
	// Total number of texture blend weights we're using
	check(InParams.TotalWeightCount > 1);
	check(InParams.TotalWeightCount <= MeshPaintDefs::MaxSupportedWeights);

	// True if we should assume the last weight index is composed of one minus the sum of all
	// of the other weights.  This effectively allows an additional weight with no extra memory
	// used, but potentially requires extra pixel shader instructions to render.
	//
	// NOTE: If you change the default here, remember to update the MeshPaintWindow UI and strings
	//
	// NOTE: Materials must be authored to match the following assumptions!
	const bool bUsingOneMinusTotal =
		InParams.TotalWeightCount == 2 ||		// Two textures: Use a lerp() in pixel shader (single value)
		InParams.TotalWeightCount == 5;			// Five texture: Requires 1.0-sum( R+G+B+A ) in shader
	check(bUsingOneMinusTotal || InParams.TotalWeightCount <= MeshPaintDefs::MaxSupportedPhysicalWeights);

	// Prefer to use RG/RGB instead of AR/ARG when we're only using 2/3 physical weights
	const int32 TotalPhysicalWeights = bUsingOneMinusTotal ? InParams.TotalWeightCount - 1 : InParams.TotalWeightCount;
	const bool bUseColorAlpha =
		TotalPhysicalWeights != 2 &&			// Two physical weights: Use RG instead of AR
		TotalPhysicalWeights != 3;				// Three physical weights: Use RGB instead of ARG

	// Index of the blend weight that we're painting
	check(InParams.PaintWeightIndex >= 0 && InParams.PaintWeightIndex < MeshPaintDefs::MaxSupportedWeights);

	// Convert the color value to an array of weights
	float Weights[MeshPaintDefs::MaxSupportedWeights];
	{
		for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
		{
			if (CurWeightIndex == TotalPhysicalWeights)
			{
				// This weight's value is one minus the sum of all previous weights
				float OtherWeightsTotal = 0.0f;
				for (int32 OtherWeightIndex = 0; OtherWeightIndex < CurWeightIndex; ++OtherWeightIndex)
				{
					OtherWeightsTotal += Weights[OtherWeightIndex];
				}
				Weights[CurWeightIndex] = 1.0f - OtherWeightsTotal;
			}
			else
			{
				switch (CurWeightIndex)
				{
				case 0:
					Weights[CurWeightIndex] = bUseColorAlpha ? OldColor.A : OldColor.R;
					break;

				case 1:
					Weights[CurWeightIndex] = bUseColorAlpha ? OldColor.R : OldColor.G;
					break;

				case 2:
					Weights[CurWeightIndex] = bUseColorAlpha ? OldColor.G : OldColor.B;
					break;

				case 3:
					check(bUseColorAlpha);
					Weights[CurWeightIndex] = OldColor.B;
					break;
				}
			}
		}
	}

	// Go ahead any apply paint!	
	Weights[InParams.PaintWeightIndex] += PaintAmount;
	Weights[InParams.PaintWeightIndex] = FMath::Clamp(Weights[InParams.PaintWeightIndex], 0.0f, 1.0f);
	

	// Now renormalize all of the other weights	
	float OtherWeightsTotal = 0.0f;
	for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
	{
		if (CurWeightIndex != InParams.PaintWeightIndex)
		{
			OtherWeightsTotal += Weights[CurWeightIndex];
		}
	}
	const float NormalizeTarget = 1.0f - Weights[InParams.PaintWeightIndex];
	for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
	{
		if (CurWeightIndex != InParams.PaintWeightIndex)
		{
			if (OtherWeightsTotal == 0.0f)
			{
				Weights[CurWeightIndex] = NormalizeTarget / (InParams.TotalWeightCount - 1);
			}
			else
			{
				Weights[CurWeightIndex] = Weights[CurWeightIndex] / OtherWeightsTotal * NormalizeTarget;
			}
		}
	}

	// The total of the weights should now always equal 1.0	
	float WeightsTotal = 0.0f;
	for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
	{
		WeightsTotal += Weights[CurWeightIndex];
	}
	check(FMath::IsNearlyEqual(WeightsTotal, 1.0f, 0.01f));
	
	// Convert the weights back to a color value	
	for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
	{
		// We can skip the non-physical weights as it's already baked into the others
		if (CurWeightIndex != TotalPhysicalWeights)
		{
			switch (CurWeightIndex)
			{
			case 0:
				if (bUseColorAlpha)
				{
					NewColor.A = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.R = Weights[CurWeightIndex];
				}
				break;

			case 1:
				if (bUseColorAlpha)
				{
					NewColor.R = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.G = Weights[CurWeightIndex];
				}
				break;

			case 2:
				if (bUseColorAlpha)
				{
					NewColor.G = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.B = Weights[CurWeightIndex];
				}
				break;

			case 3:
				NewColor.B = Weights[CurWeightIndex];
				break;
			}
		}
	}	
}

FLinearColor MeshPaintHelpers::GenerateColorForTextureWeight(const int32 NumWeights, const int32 WeightIndex)
{
	const bool bUsingOneMinusTotal =
		NumWeights == 2 ||		// Two textures: Use a lerp() in pixel shader (single value)
		NumWeights == 5;			// Five texture: Requires 1.0-sum( R+G+B+A ) in shader
	check(bUsingOneMinusTotal || NumWeights <= MeshPaintDefs::MaxSupportedPhysicalWeights);

	// Prefer to use RG/RGB instead of AR/ARG when we're only using 2/3 physical weights
	const int32 TotalPhysicalWeights = bUsingOneMinusTotal ? NumWeights - 1 : NumWeights;
	const bool bUseColorAlpha =
		TotalPhysicalWeights != 2 &&			// Two physical weights: Use RG instead of AR
		TotalPhysicalWeights != 3;				// Three physical weights: Use RGB instead of ARG

												// Index of the blend weight that we're painting
	check(WeightIndex >= 0 && WeightIndex < MeshPaintDefs::MaxSupportedWeights);

	// Convert the color value to an array of weights
	float Weights[MeshPaintDefs::MaxSupportedWeights];
	{
		for (int32 CurWeightIndex = 0; CurWeightIndex < NumWeights; ++CurWeightIndex)
		{
			if (CurWeightIndex == TotalPhysicalWeights)
			{
				// This weight's value is one minus the sum of all previous weights
				float OtherWeightsTotal = 0.0f;
				for (int32 OtherWeightIndex = 0; OtherWeightIndex < CurWeightIndex; ++OtherWeightIndex)
				{
					OtherWeightsTotal += Weights[OtherWeightIndex];
				}
				Weights[CurWeightIndex] = 1.0f - OtherWeightsTotal;
			}
			else
			{
				if (CurWeightIndex == WeightIndex)
				{
					Weights[CurWeightIndex] = 1.0f;
				}
				else
				{
					Weights[CurWeightIndex] = 0.0f;
				}
			}
		}
	}

	FLinearColor NewColor(FLinearColor::Black);
	// Convert the weights back to a color value	
	for (int32 CurWeightIndex = 0; CurWeightIndex < NumWeights; ++CurWeightIndex)
	{
		// We can skip the non-physical weights as it's already baked into the others
		if (CurWeightIndex != TotalPhysicalWeights)
		{
			switch (CurWeightIndex)
			{
			case 0:
				if (bUseColorAlpha)
				{
					NewColor.A = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.R = Weights[CurWeightIndex];
				}
				break;

			case 1:
				if (bUseColorAlpha)
				{
					NewColor.R = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.G = Weights[CurWeightIndex];
				}
				break;

			case 2:
				if (bUseColorAlpha)
				{
					NewColor.G = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.B = Weights[CurWeightIndex];
				}
				break;

			case 3:
				NewColor.B = Weights[CurWeightIndex];
				break;
			}
		}
	}

	return NewColor;
}

float MeshPaintHelpers::ComputePaintMultiplier(float SquaredDistanceToVertex2D, float BrushStrength, float BrushInnerRadius, float BrushRadialFalloff, float BrushInnerDepth, float BrushDepthFallof, float VertexDepthToBrush)
{
	float PaintAmount = 1.0f;
	
	// Compute the actual distance
	float DistanceToVertex2D = 0.0f;
	if (SquaredDistanceToVertex2D > KINDA_SMALL_NUMBER)
	{
		DistanceToVertex2D = FMath::Sqrt(SquaredDistanceToVertex2D);
	}

	// Apply radial-based falloff
	if (DistanceToVertex2D > BrushInnerRadius)
	{
		const float RadialBasedFalloff = (DistanceToVertex2D - BrushInnerRadius) / BrushRadialFalloff;
		PaintAmount *= 1.0f - RadialBasedFalloff;
	}	

	// Apply depth-based falloff	
	if (VertexDepthToBrush > BrushInnerDepth)
	{
		const float DepthBasedFalloff = (VertexDepthToBrush - BrushInnerDepth) / BrushDepthFallof;
		PaintAmount *= 1.0f - DepthBasedFalloff;
	}

	PaintAmount *= BrushStrength;

	return PaintAmount;
}

bool MeshPaintHelpers::IsPointInfluencedByBrush(const FVector& InPosition, const FMeshPaintParameters& InParams, float& OutSquaredDistanceToVertex2D, float& OutVertexDepthToBrush)
{
	// Project the vertex into the plane of the brush
	FVector BrushSpaceVertexPosition = InParams.InverseBrushToWorldMatrix.TransformPosition(InPosition);
	FVector2D BrushSpaceVertexPosition2D(BrushSpaceVertexPosition.X, BrushSpaceVertexPosition.Y);

	// Is the brush close enough to the vertex to paint?
	const float SquaredDistanceToVertex2D = BrushSpaceVertexPosition2D.SizeSquared();
	if (SquaredDistanceToVertex2D <= InParams.SquaredBrushRadius)
	{
		// OK the vertex is overlapping the brush in 2D space, but is it too close or
		// two far (depth wise) to be influenced?
		const float VertexDepthToBrush = FMath::Abs(BrushSpaceVertexPosition.Z);
		if (VertexDepthToBrush <= InParams.BrushDepth)
		{
			OutSquaredDistanceToVertex2D = SquaredDistanceToVertex2D;
			OutVertexDepthToBrush = VertexDepthToBrush;
			return true;
		}
	}

	return false;
}

bool MeshPaintHelpers::IsPointInfluencedByBrush(const FVector2D& BrushSpacePosition, const float BrushRadius, float& OutInRangeValue)
{
	const float DistanceToBrush = BrushSpacePosition.SizeSquared();
	if (DistanceToBrush <= BrushRadius)
	{
		OutInRangeValue = DistanceToBrush / BrushRadius;
		return true;
	}

	return false;
}

bool MeshPaintHelpers::RetrieveViewportPaintRays(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI, TArray<FPaintRay>& OutPaintRays)
{
	checkf(View && Viewport && PDI, TEXT("Invalid Viewport data"));
	FEditorViewportClient* ViewportClient = (FEditorViewportClient*)Viewport->GetClient();
	checkf(ViewportClient != nullptr, TEXT("Unable to retrieve viewport client"));

	if (ViewportClient->IsPerspective())
	{

		// Make sure the cursor is visible OR we're flood filling.  No point drawing a paint cue when there's no cursor.
		if (Viewport->IsCursorVisible())
		{
			if (!PDI->IsHitTesting())
			{
				// Grab the mouse cursor position
				FIntPoint MousePosition;
				Viewport->GetMousePos(MousePosition);

				// Is the mouse currently over the viewport? or flood filling
				if ((MousePosition.X >= 0 && MousePosition.Y >= 0 && MousePosition.X < (int32)Viewport->GetSizeXY().X && MousePosition.Y < (int32)Viewport->GetSizeXY().Y))
				{
					// Compute a world space ray from the screen space mouse coordinates
					FViewportCursorLocation MouseViewportRay(View, ViewportClient, MousePosition.X, MousePosition.Y);

					FPaintRay& NewPaintRay = *new(OutPaintRays) FPaintRay();
					NewPaintRay.CameraLocation = View->ViewMatrices.GetViewOrigin();
					NewPaintRay.RayStart = MouseViewportRay.GetOrigin();
					NewPaintRay.RayDirection = MouseViewportRay.GetDirection();
					NewPaintRay.ViewportInteractor = nullptr;
				}
			}
		}
	}

	return false;
}

uint32 MeshPaintHelpers::GetVertexColorBufferSize(UMeshComponent* MeshComponent, int32 LODIndex, bool bInstance)
{
	checkf(MeshComponent != nullptr, TEXT("Invalid static mesh component ptr"));
	
	uint32 SizeInBytes = 0;

	// Retrieve component instance vertex color buffer size
	if (bInstance)
	{
		if(UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
		{
			if (StaticMeshComponent->LODData.IsValidIndex(LODIndex))
			{
				const FStaticMeshComponentLODInfo& InstanceMeshLODInfo = StaticMeshComponent->LODData[LODIndex];
				if (InstanceMeshLODInfo.OverrideVertexColors)
				{
					SizeInBytes = InstanceMeshLODInfo.OverrideVertexColors->GetAllocatedSize();
				}
			}
		}
		else if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(MeshComponent))
		{
			if (SkinnedMeshComponent->LODInfo.IsValidIndex(LODIndex))
			{
				const FSkelMeshComponentLODInfo& LODInfo = SkinnedMeshComponent->LODInfo[LODIndex];
				if (LODInfo.OverrideVertexColors)
				{
					SizeInBytes = LODInfo.OverrideVertexColors->GetAllocatedSize();
				}
			}
		}
	}
	// Retrieve static mesh asset vertex color buffer size
	else
	{
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
		{
			UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			checkf(StaticMesh != nullptr, TEXT("Invalid static mesh ptr"));
			if (StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex))
			{
				// count the base mesh color data
				FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[LODIndex];
				SizeInBytes = LODModel.VertexBuffers.ColorVertexBuffer.GetAllocatedSize();
			}
		}
		else if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(MeshComponent))
		{
			FSkeletalMeshRenderData* RenderData = SkinnedMeshComponent->GetSkeletalMeshRenderData();
			if (RenderData && RenderData->LODRenderData.IsValidIndex(LODIndex))
			{
				SizeInBytes = RenderData->LODRenderData[LODIndex].StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize();
			}
		}
	}
	
	return SizeInBytes;
}

TArray<FVector> MeshPaintHelpers::GetVerticesForLOD( const UStaticMesh* StaticMesh, int32 LODIndex)
{
	checkf(StaticMesh != nullptr, TEXT("Invalid static mesh ptr"));

	// Retrieve mesh vertices from Static mesh render data 
	TArray<FVector> Vertices;
	if (StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex))
	{
		const FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[LODIndex];
		const FPositionVertexBuffer* VertexBuffer = &LODModel.VertexBuffers.PositionVertexBuffer;
		const uint32 NumVertices = VertexBuffer->GetNumVertices();
		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			Vertices.Add((FVector)VertexBuffer->VertexPosition(VertexIndex));
		}		
	}
	return Vertices;
}

TArray<FColor> MeshPaintHelpers::GetColorDataForLOD( const UStaticMesh* StaticMesh, int32 LODIndex)
{
	checkf(StaticMesh != nullptr, TEXT("Invalid static mesh ptr"));
	// Retrieve mesh vertex colors from Static mesh render data 
	TArray<FColor> Colors;
	if (StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex))
	{
		const FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[LODIndex];
		const FColorVertexBuffer& ColorBuffer = LODModel.VertexBuffers.ColorVertexBuffer;
		const uint32 NumColors = ColorBuffer.GetNumVertices();
		for (uint32 ColorIndex = 0; ColorIndex < NumColors; ++ColorIndex)
		{
			Colors.Add(ColorBuffer.VertexColor(ColorIndex));
		}
	}
	return Colors;
}

TArray<FColor> MeshPaintHelpers::GetInstanceColorDataForLOD(const UStaticMeshComponent* MeshComponent, int32 LODIndex)
{
	checkf(MeshComponent != nullptr, TEXT("Invalid static mesh component ptr"));
	TArray<FColor> Colors;
	
	// Retrieve mesh vertex colors from Static Mesh component instance data
	if (MeshComponent->LODData.IsValidIndex(LODIndex))
	{
		const FStaticMeshComponentLODInfo& ComponentLODInfo = MeshComponent->LODData[LODIndex];
		const FColorVertexBuffer* ColorBuffer = ComponentLODInfo.OverrideVertexColors;
		if (ColorBuffer)
		{
			const uint32 NumColors = ColorBuffer->GetNumVertices();
			for (uint32 ColorIndex = 0; ColorIndex < NumColors; ++ColorIndex)
			{
				Colors.Add(ColorBuffer->VertexColor(ColorIndex));
			}
		}
	}

	return Colors;
}

void MeshPaintHelpers::SetInstanceColorDataForLOD(UStaticMeshComponent* MeshComponent, int32 LODIndex, const TArray<FColor>& Colors)
{
	checkf(MeshComponent != nullptr, TEXT("Invalid static mesh component ptr"));

	const UStaticMesh* Mesh = MeshComponent->GetStaticMesh();
	if (Mesh)
	{
		const FStaticMeshLODResources& RenderData = Mesh->GetRenderData()->LODResources[LODIndex];
		FStaticMeshComponentLODInfo& ComponentLodInfo = MeshComponent->LODData[LODIndex];		

		// First release existing buffer
		if (ComponentLodInfo.OverrideVertexColors)
		{
			ComponentLodInfo.ReleaseOverrideVertexColorsAndBlock();
		}

		// If we are adding colors to LOD > 0 we flag the component to have per-lod painted mesh colors
		if (LODIndex > 0)
		{
			MeshComponent->bCustomOverrideVertexColorPerLOD = true;			
		}

		// Initialize vertex buffer from given colors
		ComponentLodInfo.OverrideVertexColors = new FColorVertexBuffer;
		ComponentLodInfo.OverrideVertexColors->InitFromColorArray(Colors);
		
		//Update the cache painted vertices
		ComponentLodInfo.PaintedVertices.Empty();
		MeshComponent->CachePaintedDataIfNecessary();

		BeginInitResource(ComponentLodInfo.OverrideVertexColors);
	}
}

void MeshPaintHelpers::SetInstanceColorDataForLOD(UStaticMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor )
{
	checkf(MeshComponent != nullptr, TEXT("Invalid static mesh component ptr"));

	const UStaticMesh* Mesh = MeshComponent->GetStaticMesh();
	if (Mesh)
	{
		const FStaticMeshLODResources& RenderData = Mesh->GetRenderData()->LODResources[LODIndex];
		// Ensure we have enough LOD data structs
		MeshComponent->SetLODDataCount(LODIndex + 1, MeshComponent->LODData.Num());
		FStaticMeshComponentLODInfo& ComponentLodInfo = MeshComponent->LODData[LODIndex];

		if (MaskColor == FColor::White)
		{
			// First release existing buffer
			if (ComponentLodInfo.OverrideVertexColors)
			{
				ComponentLodInfo.ReleaseOverrideVertexColorsAndBlock();
			}

			// If we are adding colors to LOD > 0 we flag the component to have per-lod painted mesh colors
			if (LODIndex > 0)
			{
				MeshComponent->bCustomOverrideVertexColorPerLOD = true;
			}

			// Initialize vertex buffer from given color
			ComponentLodInfo.OverrideVertexColors = new FColorVertexBuffer;
			ComponentLodInfo.OverrideVertexColors->InitFromSingleColor(FillColor, RenderData.GetNumVertices());			
		}
		else
		{
			const FStaticMeshLODResources& LODModel = MeshComponent->GetStaticMesh()->GetRenderData()->LODResources[LODIndex];
			/** If there is an actual mask apply it to Fill Color when changing the per-vertex color */
			if (ComponentLodInfo.OverrideVertexColors)
			{
				const uint32 NumVertices = ComponentLodInfo.OverrideVertexColors->GetNumVertices();
				for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{					
					ApplyFillWithMask(ComponentLodInfo.OverrideVertexColors->VertexColor(VertexIndex), MaskColor, FillColor);
				}
			}
			else
			{
				// Initialize vertex buffer from given color
				ComponentLodInfo.OverrideVertexColors = new FColorVertexBuffer;
				FColor NewFillColor(EForceInit::ForceInitToZero);
				ApplyFillWithMask(NewFillColor, MaskColor, FillColor);
				ComponentLodInfo.OverrideVertexColors->InitFromSingleColor(NewFillColor, RenderData.GetNumVertices());
			}
		}

		//Update the cache painted vertices
		ComponentLodInfo.PaintedVertices.Empty();
		MeshComponent->CachePaintedDataIfNecessary();

		BeginInitResource(ComponentLodInfo.OverrideVertexColors);
	}
}

void MeshPaintHelpers::FillStaticMeshVertexColors(UStaticMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor)
{
	UStaticMesh* Mesh = MeshComponent->GetStaticMesh();
	if (Mesh)
	{
		const int32 NumLODs = Mesh->GetNumLODs();
		if (LODIndex < NumLODs)
		{
			if (LODIndex == -1)
			{
				for (LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
				{
					MeshPaintHelpers::SetInstanceColorDataForLOD(MeshComponent, LODIndex, FillColor, MaskColor);
				}
			}
			else
			{
				MeshPaintHelpers::SetInstanceColorDataForLOD(MeshComponent, LODIndex, FillColor, MaskColor);
			}
		}
	}
}

void MeshPaintHelpers::FillSkeletalMeshVertexColors(USkeletalMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor)
{
	TUniquePtr< FSkinnedMeshComponentRecreateRenderStateContext > RecreateRenderStateContext;
	USkeletalMesh* Mesh = MeshComponent->GetSkeletalMeshAsset();
	if (Mesh)
	{
		// Dirty the mesh
		Mesh->SetFlags(RF_Transactional);
		Mesh->Modify();
		Mesh->SetHasVertexColors(true);
		Mesh->SetVertexColorGuid(FGuid::NewGuid());

		// Release the static mesh's resources.
		Mesh->ReleaseResources();

		// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
		// allocated, and potentially accessing the UStaticMesh.
		Mesh->ReleaseResourcesFence.Wait();

		const int32 NumLODs = Mesh->GetLODNum();
		if (NumLODs > 0)
		{
			RecreateRenderStateContext = MakeUnique<FSkinnedMeshComponentRecreateRenderStateContext>(Mesh);
			// TODO: Apply to LODIndex only (or all if set to -1). This requires some extra refactoring
			// because currently all LOD data is being released above.
			for (LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				MeshPaintHelpers::SetColorDataForLOD(Mesh, LODIndex, FillColor, MaskColor);
				PropagateVertexPaintToAsset(Mesh, LODIndex);
			}
			Mesh->InitResources();
		}
	}
}

void MeshPaintHelpers::SetColorDataForLOD(USkeletalMesh* SkeletalMesh, int32 LODIndex, const FColor FillColor, const FColor MaskColor )
{
	checkf(SkeletalMesh != nullptr, TEXT("Invalid Skeletal Mesh Ptr"));
	FSkeletalMeshRenderData* Resource = SkeletalMesh->GetResourceForRendering();
	if (Resource && Resource->LODRenderData.IsValidIndex(LODIndex))
	{
		FSkeletalMeshLODRenderData& LODData = Resource->LODRenderData[LODIndex];

		if (MaskColor == FColor::White)
		{
			LODData.StaticVertexBuffers.ColorVertexBuffer.InitFromSingleColor(FillColor, LODData.GetNumVertices());
		}
		else
		{
			/** If there is an actual mask apply it to Fill Color when changing the per-vertex color */
			const uint32 NumVertices = LODData.StaticVertexBuffers.ColorVertexBuffer.GetNumVertices();
			for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				ApplyFillWithMask(LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex), MaskColor, FillColor);
			}
		}

		BeginInitResource(&LODData.StaticVertexBuffers.ColorVertexBuffer);
	}	

	checkf(SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex), TEXT("Invalid Imported Model index for vertex painting"));
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	const uint32 NumVertices = LODModel.NumVertices;
	for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		int32 SectionIndex = INDEX_NONE;
		int32 SectionVertexIndex = INDEX_NONE;
		LODModel.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);
		/** If there is an actual mask apply it to Fill Color when changing the per-vertex color */
		ApplyFillWithMask(LODModel.Sections[SectionIndex].SoftVertices[SectionVertexIndex].Color, MaskColor, FillColor);
	}

	if (!SkeletalMesh->GetLODInfo(LODIndex)->bHasPerLODVertexColors)
	{
		SkeletalMesh->GetLODInfo(LODIndex)->bHasPerLODVertexColors = true;
	}
}

void MeshPaintHelpers::ApplyFillWithMask(FColor& InOutColor, const FColor& MaskColor, const FColor& FillColor)
{
	InOutColor.R = ((InOutColor.R & (~MaskColor.R)) | (FillColor.R & MaskColor.R));
	InOutColor.G = ((InOutColor.G & (~MaskColor.G)) | (FillColor.G & MaskColor.G));
	InOutColor.B = ((InOutColor.B & (~MaskColor.B)) | (FillColor.B & MaskColor.B));
	InOutColor.A = ((InOutColor.A & (~MaskColor.A)) | (FillColor.A & MaskColor.A));
}

void MeshPaintHelpers::ImportVertexColorsFromTexture(UMeshComponent* MeshComponent)
{
	checkf(MeshComponent != nullptr, TEXT("Invalid mesh component ptr"));

	// Get TGA texture filepath
	FString ChosenFilename("");
	FString ExtensionStr;
	ExtensionStr += TEXT("TGA Files|*.tga|");

	FString PromptTitle("Pick TGA Texture File");

	// First, display the file open dialog for selecting the file.
	TArray<FString> Filenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpen = false;
	if (DesktopPlatform)
	{
		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			PromptTitle,
			TEXT(""),
			TEXT(""),
			*ExtensionStr,
			EFileDialogFlags::None,
			Filenames
		);
	}

	if (bOpen && Filenames.Num() == 1)
	{
		// Valid file name picked
		const FString FileName = Filenames[0];
		UTexture2D* ColorTexture = ImportObject<UTexture2D>(GEngine, NAME_None, RF_Public, *FileName, nullptr, nullptr, TEXT("NOMIPMAPS=1 NOCOMPRESSION=1"));

		if (ColorTexture && ColorTexture->Source.GetFormat() == TSF_BGRA8)
		{
			// Have a valid texture, now need user to specify options for importing
			TSharedRef<SWindow> Window = SNew(SWindow)
				.Title(FText::FromString(TEXT("Vertex Color Import Options")))
				.SizingRule(ESizingRule::Autosized);

			TSharedPtr<SImportVertexColorOptions> OptionsWindow = SNew(SImportVertexColorOptions).WidgetWindow(Window)
				.WidgetWindow(Window)
				.Component(MeshComponent)
				.FullPath(FText::FromString(ChosenFilename));

			Window->SetContent
			(
				OptionsWindow->AsShared()
			);

			TSharedPtr<SWindow> ParentWindow;
			if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
			{
				IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
				ParentWindow = MainFrame.GetParentWindow();
			}
			FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

			if (OptionsWindow->ShouldImport())
			{
				// Options specified and start importing
				UVertexColorImportOptions* Options = OptionsWindow->GetOptions();
				
				if (MeshComponent->IsA<UStaticMeshComponent>())				
				{ 
					UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent);
					if (StaticMeshComponent)
					{
						if (Options->bImportToInstance)
						{
							// Import colors to static mesh / component
							ImportVertexColorsToStaticMeshComponent(StaticMeshComponent, Options, ColorTexture);
						}
						else
						{
							if (StaticMeshComponent->GetStaticMesh())
							{
								ImportVertexColorsToStaticMesh(StaticMeshComponent->GetStaticMesh(), Options, ColorTexture);
							}
						}						
					}					
				}
				else if (MeshComponent->IsA<USkeletalMeshComponent>())
				{
					USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent);

					if (SkeletalMeshComponent->GetSkeletalMeshAsset())
					{
						// Import colors to skeletal mesh
						ImportVertexColorsToSkeletalMesh(SkeletalMeshComponent->GetSkeletalMeshAsset(), Options, ColorTexture);
					}			
				}
			}
		}
		else if (!ColorTexture)
		{
			// Unable to import file
		}
		else if (ColorTexture && ColorTexture->Source.GetFormat() != TSF_BGRA8)
		{
			// Able to import file but incorrect format
		}
	}
}

void MeshPaintHelpers::SetViewportColorMode(EMeshPaintColorViewMode ColorViewMode, FEditorViewportClient* ViewportClient)
{
	if (ViewportClient->IsPerspective())
	{
		// Update viewport show flags
		{
			// show flags forced on during vertex color modes
			if (ColorViewMode == EMeshPaintColorViewMode::Normal)
			{
				ColorViewMode = EMeshPaintColorViewMode::Normal;
			}

			if (ColorViewMode == EMeshPaintColorViewMode::Normal)
			{
				if (ViewportClient->EngineShowFlags.VertexColors)
				{
					// If we're transitioning to normal mode then restore the backup
					// Clear the flags relevant to vertex color modes
					ViewportClient->EngineShowFlags.SetVertexColors(false);

					// Restore the vertex color mode flags that were set when we last entered vertex color mode
					ApplyViewMode(ViewportClient->GetViewMode(), ViewportClient->IsPerspective(), ViewportClient->EngineShowFlags);
					MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Color);
				}
			}
			else
			{
				ViewportClient->EngineShowFlags.SetMaterials(true);
				ViewportClient->EngineShowFlags.SetLighting(false);
				ViewportClient->EngineShowFlags.SetBSPTriangles(true);
				ViewportClient->EngineShowFlags.SetVertexColors(true);
				ViewportClient->EngineShowFlags.SetPostProcessing(false);
				ViewportClient->EngineShowFlags.SetHMDDistortion(false);

				switch (ColorViewMode)
				{
					case EMeshPaintColorViewMode::RGB:
					{
						MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Color);
					}
					break;

					case EMeshPaintColorViewMode::Alpha:
					{
						MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Alpha);
					}
					break;

					case EMeshPaintColorViewMode::Red:
					{
						MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Red);
					}
					break;

					case EMeshPaintColorViewMode::Green:
					{
						MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Green);
					}
					break;

					case EMeshPaintColorViewMode::Blue:
					{
						MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Blue);
					}
					break;
				}
			}
		}
	}
}

void MeshPaintHelpers::SetRealtimeViewport(bool bRealtime)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr< IAssetViewport > ViewportWindow = LevelEditorModule.GetFirstActiveViewport();
	const bool bRememberCurrentState = false;
	if (ViewportWindow.IsValid())
	{
		FEditorViewportClient &Viewport = ViewportWindow->GetAssetViewportClient();
		if (Viewport.IsPerspective())
		{
			const FText SystemDisplayName = NSLOCTEXT("MeshPaint", "RealtimeOverrideMessage_MeshPaint", "Mesh Paint");
			if (bRealtime)
			{
				Viewport.AddRealtimeOverride(bRealtime, SystemDisplayName);
			}
			else
			{
				Viewport.RemoveRealtimeOverride(SystemDisplayName);
			}
		}
	}
}


void MeshPaintHelpers::ForceRenderMeshLOD(UMeshComponent* Component, int32 LODIndex)
{
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
		StaticMeshComponent->ForcedLodModel = LODIndex + 1;
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component))
	{
		SkeletalMeshComponent->SetForcedLOD(LODIndex + 1);
	}	
}

void MeshPaintHelpers::ClearMeshTextureOverrides(const IMeshPaintGeometryAdapter& GeometryInfo, UMeshComponent* InMeshComponent)
{
	if (InMeshComponent != nullptr)
	{
		TArray<UTexture*> UsedTextures;
		InMeshComponent->GetUsedTextures(/*out*/ UsedTextures, EMaterialQualityLevel::High);

		for (UTexture* Texture : UsedTextures)
		{
			if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
			{
				GeometryInfo.ApplyOrRemoveTextureOverride(Texture2D, nullptr);
			}
		}
	}
}

void MeshPaintHelpers::ApplyVertexColorsToAllLODs(IMeshPaintGeometryAdapter& GeometryInfo, UMeshComponent* InMeshComponent)
{
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(InMeshComponent))
	{
		ApplyVertexColorsToAllLODs(GeometryInfo, StaticMeshComponent);
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InMeshComponent))
	{
		ApplyVertexColorsToAllLODs(GeometryInfo, SkeletalMeshComponent);
	}
}

void MeshPaintHelpers::ApplyVertexColorsToAllLODs(IMeshPaintGeometryAdapter& GeometryInfo, UStaticMeshComponent* StaticMeshComponent)
{
	// If a static mesh component was found, apply LOD0 painting to all lower LODs.
	if (!StaticMeshComponent || !StaticMeshComponent->GetStaticMesh())
	{
		return;
	}

	if (StaticMeshComponent->LODData.Num() < 1)
	{
		//We need at least some painting on the base LOD to apply it to the lower LODs
		return;
	}

	//Make sure we have something paint in the LOD 0 to apply it to all lower LODs.
	if (StaticMeshComponent->LODData[0].OverrideVertexColors == nullptr && StaticMeshComponent->LODData[0].PaintedVertices.Num() <= 0)
	{
		return;
	}

	StaticMeshComponent->bCustomOverrideVertexColorPerLOD = false;

	uint32 NumLODs = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources.Num();
	StaticMeshComponent->Modify();

	// Ensure LODData has enough entries in it, free not required.
	StaticMeshComponent->SetLODDataCount(NumLODs, StaticMeshComponent->LODData.Num());
	for (uint32 i = 1; i < NumLODs; ++i)
	{
		FStaticMeshComponentLODInfo* CurrInstanceMeshLODInfo = &StaticMeshComponent->LODData[i];
		FStaticMeshLODResources& CurrRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[i];
		// Destroy the instance vertex  color array if it doesn't fit
		if (CurrInstanceMeshLODInfo->OverrideVertexColors
			&& CurrInstanceMeshLODInfo->OverrideVertexColors->GetNumVertices() != CurrRenderData.GetNumVertices())
		{
			CurrInstanceMeshLODInfo->ReleaseOverrideVertexColorsAndBlock();
		}

		if (CurrInstanceMeshLODInfo->OverrideVertexColors)
		{
			CurrInstanceMeshLODInfo->BeginReleaseOverrideVertexColors();
		}
		else
		{
			// Setup the instance vertex color array if we don't have one yet
			CurrInstanceMeshLODInfo->OverrideVertexColors = new FColorVertexBuffer;
		}
	}

	FlushRenderingCommands();
	
	const FStaticMeshComponentLODInfo& SourceCompLODInfo = StaticMeshComponent->LODData[0];
	const FStaticMeshLODResources& SourceRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[0];
	for (uint32 i = 1; i < NumLODs; ++i)
	{
		FStaticMeshComponentLODInfo& CurCompLODInfo = StaticMeshComponent->LODData[i];
		FStaticMeshLODResources& CurRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[i];

		check(CurCompLODInfo.OverrideVertexColors);
		check(SourceCompLODInfo.OverrideVertexColors);

		TArray<FColor> NewOverrideColors;
		
		RemapPaintedVertexColors(
			SourceCompLODInfo.PaintedVertices,
			SourceCompLODInfo.OverrideVertexColors,
			SourceRenderData.VertexBuffers.PositionVertexBuffer,
			SourceRenderData.VertexBuffers.StaticMeshVertexBuffer,
			CurRenderData.VertexBuffers.PositionVertexBuffer,
			&CurRenderData.VertexBuffers.StaticMeshVertexBuffer,
			NewOverrideColors
		);
		
		if (NewOverrideColors.Num())
		{
			CurCompLODInfo.OverrideVertexColors->InitFromColorArray(NewOverrideColors);
		}

		// Initialize the vert. colors
		BeginInitResource(CurCompLODInfo.OverrideVertexColors);
	}
}

bool MeshPaintHelpers::TryGetNumberOfLODs(const UMeshComponent* MeshComponent, int32& OutNumLODs)
{
	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
	{
		const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (StaticMesh != nullptr)
		{
			OutNumLODs = StaticMesh->GetNumLODs();
			return true;
		}
	}
	else if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent))
	{
		const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh != nullptr)
		{
			OutNumLODs = SkeletalMesh->GetLODNum();
			return true;
		}
	}

	return false;
}

int32 MeshPaintHelpers::GetNumberOfLODs(const UMeshComponent* MeshComponent)
{
	int32 NumLODs = 1;
	TryGetNumberOfLODs(MeshComponent, NumLODs);
	return NumLODs;
}

int32 MeshPaintHelpers::GetNumberOfUVs(const UMeshComponent* MeshComponent, int32 LODIndex)
{
	int32 NumUVs = 0;

	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
	{
		const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (StaticMesh != nullptr && StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex))
		{
			NumUVs = StaticMesh->GetRenderData()->LODResources[LODIndex].GetNumTexCoords();
		}
	}
	else if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent))
	{
		const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh != nullptr && SkeletalMesh->GetResourceForRendering() && SkeletalMesh->GetResourceForRendering()->LODRenderData.IsValidIndex(LODIndex))
		{
			NumUVs = SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex].GetNumTexCoords();
		}
	}

	return NumUVs;
}

bool MeshPaintHelpers::DoesMeshComponentContainPerLODColors(const UMeshComponent* MeshComponent)
{
	bool bPerLODColors = false;

	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
	{
		bPerLODColors = StaticMeshComponent->bCustomOverrideVertexColorPerLOD;
		
		bool bInstancedLODColors = false;
		if (bPerLODColors)
        {

		  const int32 NumLODs = StaticMeshComponent->LODData.Num();
		  
		  for (int32 LODIndex = 1; LODIndex < NumLODs; ++LODIndex)
		  {
			  if (StaticMeshComponent->LODData[LODIndex].PaintedVertices.Num() > 0)
			  {
				  bInstancedLODColors = true;
				  break;
			  }
		  }
		}

		bPerLODColors = bPerLODColors && bInstancedLODColors;
	}
	else if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent))
	{
		USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh)
		{
			// Only check LOD level 1 and above
			for (int32 LODIndex = 1, NumLODs = SkeletalMesh->GetLODNum(); LODIndex < NumLODs; ++LODIndex)
			{
				const FSkeletalMeshLODInfo* Info = SkeletalMesh->GetLODInfo(LODIndex);
				if (Info->bHasPerLODVertexColors)
				{
					bPerLODColors = true;
					break;
				}
			}
		}
	}

	return bPerLODColors;
}

void MeshPaintHelpers::GetInstanceColorDataInfo(const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, int32& OutTotalInstanceVertexColorBytes)
{
	checkf(StaticMeshComponent, TEXT("Invalid StaticMeshComponent"));
	OutTotalInstanceVertexColorBytes = 0;
	
	const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	if (StaticMesh != nullptr && StaticMesh->GetNumLODs() > (int32)LODIndex && StaticMeshComponent->LODData.IsValidIndex(LODIndex))
	{
		// count the instance color data		
		const FStaticMeshComponentLODInfo& InstanceMeshLODInfo = StaticMeshComponent->LODData[LODIndex];
		if (InstanceMeshLODInfo.OverrideVertexColors)
		{
			OutTotalInstanceVertexColorBytes += InstanceMeshLODInfo.OverrideVertexColors->GetAllocatedSize();
		}
	}
}

void MeshPaintHelpers::ImportVertexColorsToStaticMesh(UStaticMesh* StaticMesh, const UVertexColorImportOptions* Options, UTexture2D* Texture)
{
	checkf(StaticMesh && Options && Texture, TEXT("Invalid ptr"));

	// Extract color data from texture
	// todo: use GetMipImage instead of GetMipData
	TArray64<uint8> SrcMipData;
	verify( Texture->Source.GetMipData(SrcMipData, 0) );
	const uint8* MipData = SrcMipData.GetData();

	TUniquePtr< FStaticMeshComponentRecreateRenderStateContext > RecreateRenderStateContext = MakeUnique<FStaticMeshComponentRecreateRenderStateContext>(StaticMesh);
	const int32 ImportLOD = Options->LODIndex;
	FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[ImportLOD];
	
	// Dirty the mesh
	StaticMesh->Modify();

	// Release the static mesh's resources.
	StaticMesh->ReleaseResources();

	// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
	// allocated, and potentially accessing the UStaticMesh.
	StaticMesh->ReleaseResourcesFence.Wait();

	if (LODModel.VertexBuffers.ColorVertexBuffer.GetNumVertices() == 0)
	{
		// Mesh doesn't have a color vertex buffer yet!  We'll create one now.
		LODModel.VertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor::White, LODModel.GetNumVertices());

		// @todo MeshPaint: Make sure this is the best place to do this
		BeginInitResource(&LODModel.VertexBuffers.ColorVertexBuffer);
	}

	const int32 UVIndex = Options->UVIndex;
	const FColor ColorMask = Options->CreateColorMask();
	for (uint32 VertexIndex = 0; VertexIndex < LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices(); ++VertexIndex)
	{
		const FVector2D UV = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVIndex));
		LODModel.VertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = PickVertexColorFromTextureData(MipData, UV, Texture, ColorMask);
	}

	// Make sure colors are saved into raw mesh

	StaticMesh->InitResources();
}

void MeshPaintHelpers::ImportVertexColorsToStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent, const UVertexColorImportOptions* Options, UTexture2D* Texture)
{
	checkf(StaticMeshComponent && Options && Texture, TEXT("Invalid ptr"));

	// Extract color data from texture
	// todo: use GetMipImage instead of GetMipData
	TArray64<uint8> SrcMipData;
	verify( Texture->Source.GetMipData(SrcMipData, 0) );
	const uint8* MipData = SrcMipData.GetData();

	TUniquePtr< FComponentReregisterContext > ComponentReregisterContext;
	const UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh();	
	if (Mesh)
	{
		ComponentReregisterContext = MakeUnique<FComponentReregisterContext>(StaticMeshComponent);
		StaticMeshComponent->Modify();

		const int32 ImportLOD = Options->LODIndex;
		const FStaticMeshLODResources& LODModel = Mesh->GetRenderData()->LODResources[ImportLOD];

		if (!StaticMeshComponent->LODData.IsValidIndex(ImportLOD))
		{
			StaticMeshComponent->SetLODDataCount(ImportLOD + 1, StaticMeshComponent->LODData.Num());
		}		

		FStaticMeshComponentLODInfo& InstanceMeshLODInfo = StaticMeshComponent->LODData[ImportLOD];

		if (InstanceMeshLODInfo.OverrideVertexColors)
		{
			InstanceMeshLODInfo.ReleaseOverrideVertexColorsAndBlock();
		}
		
		// Setup the instance vertex color array
		InstanceMeshLODInfo.OverrideVertexColors = new FColorVertexBuffer;

		if ((int32)LODModel.VertexBuffers.ColorVertexBuffer.GetNumVertices() == LODModel.GetNumVertices())
		{
			// copy mesh vertex colors to the instance ones
			InstanceMeshLODInfo.OverrideVertexColors->InitFromColorArray(&LODModel.VertexBuffers.ColorVertexBuffer.VertexColor(0), LODModel.GetNumVertices());
		}
		else
		{
			// Original mesh didn't have any colors, so just use a default color
			InstanceMeshLODInfo.OverrideVertexColors->InitFromSingleColor(FColor::White, LODModel.GetNumVertices());
		}
		
		if (ImportLOD > 0)
		{
			StaticMeshComponent->bCustomOverrideVertexColorPerLOD = true;
		}

		const int32 UVIndex = Options->UVIndex;
		const FColor ColorMask = Options->CreateColorMask();
		for (uint32 VertexIndex = 0; VertexIndex < LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices(); ++VertexIndex)
		{
			const FVector2D UV = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVIndex));
			InstanceMeshLODInfo.OverrideVertexColors->VertexColor(VertexIndex) = PickVertexColorFromTextureData(MipData, UV, Texture, ColorMask);
		}

		//Update the cache painted vertices
		InstanceMeshLODInfo.PaintedVertices.Empty();
		StaticMeshComponent->CachePaintedDataIfNecessary();

		BeginInitResource(InstanceMeshLODInfo.OverrideVertexColors);
	}
	else
	{
		// Error
	}
}

void MeshPaintHelpers::ImportVertexColorsToSkeletalMesh(USkeletalMesh* SkeletalMesh, const UVertexColorImportOptions* Options, UTexture2D* Texture)
{
	checkf(SkeletalMesh && Options && Texture, TEXT("Invalid ptr"));

	// Extract color data from texture
	// todo: use GetMipImage instead of GetMipData
	TArray64<uint8> SrcMipData;
	verify( Texture->Source.GetMipData(SrcMipData, 0) );
	const uint8* MipData = SrcMipData.GetData();

	TUniquePtr< FSkinnedMeshComponentRecreateRenderStateContext > RecreateRenderStateContext;
	FSkeletalMeshRenderData* Resource = SkeletalMesh->GetResourceForRendering();
	const int32 ImportLOD = Options->LODIndex;
	const int32 UVIndex = Options->UVIndex;
	const FColor ColorMask = Options->CreateColorMask();
	if (Resource && Resource->LODRenderData.IsValidIndex(ImportLOD))
	{
		RecreateRenderStateContext = MakeUnique<FSkinnedMeshComponentRecreateRenderStateContext>(SkeletalMesh);
		SkeletalMesh->Modify();
		SkeletalMesh->ReleaseResources();
		SkeletalMesh->ReleaseResourcesFence.Wait();

		FSkeletalMeshLODRenderData& LODData = Resource->LODRenderData[ImportLOD];

		if (LODData.StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() == 0)
		{
			LODData.StaticVertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor::White, LODData.GetNumVertices());
			BeginInitResource(&LODData.StaticVertexBuffers.ColorVertexBuffer);
		}

		for (uint32 VertexIndex = 0; VertexIndex < LODData.GetNumVertices(); ++VertexIndex)
		{
			const FVector2D UV = FVector2D(LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVIndex));
			LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = PickVertexColorFromTextureData(MipData, UV, Texture, ColorMask);
		}
		
		SkeletalMesh->InitResources();
	}


	checkf(SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(ImportLOD), TEXT("Invalid Imported Model index for vertex painting"));
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[ImportLOD];
	const uint32 NumVertices = LODModel.NumVertices;
	for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		int32 SectionIndex = INDEX_NONE;
		int32 SectionVertexIndex = INDEX_NONE;
		LODModel.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);

		const FVector2D UV = FVector2D(LODModel.Sections[SectionIndex].SoftVertices[SectionVertexIndex].UVs[UVIndex]);
		LODModel.Sections[SectionIndex].SoftVertices[SectionVertexIndex].Color = PickVertexColorFromTextureData(MipData, UV, Texture, ColorMask);
	}

	//Make sure we change the import data so the re-import do not replace the new data
	if (SkeletalMesh->GetAssetImportData())
	{
		UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(SkeletalMesh->GetAssetImportData());
		if (ImportData && ImportData->VertexColorImportOption != EVertexColorImportOption::Ignore)
		{
			ImportData->SetFlags(RF_Transactional);
			ImportData->Modify();
			ImportData->VertexColorImportOption = EVertexColorImportOption::Ignore;
		}
	}
}

FColor MeshPaintHelpers::PickVertexColorFromTextureData(const uint8* MipData, const FVector2D& UVCoordinate, const UTexture2D* Texture, const FColor ColorMask)
{
	checkf(MipData, TEXT("Invalid texture MIP data"));
	FColor VertexColor = FColor::Black;

	if ((UVCoordinate.X >= 0.0f) && (UVCoordinate.X < 1.0f) && (UVCoordinate.Y >= 0.0f) && (UVCoordinate.Y < 1.0f))
	{
		const int32 X = Texture->GetSizeX()*UVCoordinate.X;
		const int32 Y = Texture->GetSizeY()*UVCoordinate.Y;

		const int32 Index = ((Y * Texture->GetSizeX()) + X) * 4;
		VertexColor.B = MipData[Index + 0];
		VertexColor.G = MipData[Index + 1];
		VertexColor.R = MipData[Index + 2];
		VertexColor.A = MipData[Index + 3];

		VertexColor.DWColor() &= ColorMask.DWColor();
	}

	return VertexColor;
}

bool MeshPaintHelpers::GetPerVertexPaintInfluencedVertices(FPerVertexPaintActionArgs& InArgs, TSet<int32>& InfluencedVertices)
{
	// Retrieve components world matrix
	const FMatrix& ComponentToWorldMatrix = InArgs.Adapter->GetComponentToWorldMatrix();
	
	// Compute the camera position in actor space.  We need this later to check for back facing triangles.
	const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(InArgs.CameraPosition));
	const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(InArgs.HitResult.Location));

	// @todo MeshPaint: Input vector doesn't work well with non-uniform scale
	const float BrushRadius = InArgs.BrushSettings->GetBrushRadius();
	const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(BrushRadius, 0.0f, 0.0f)).Size();
	const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

	// Get a list of unique vertices indexed by the influenced triangles
	InArgs.Adapter->GetInfluencedVertexIndices(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, InArgs.BrushSettings->bOnlyFrontFacingTriangles, InfluencedVertices);

	return (InfluencedVertices.Num() > 0);
}

bool MeshPaintHelpers::ApplyPerVertexPaintAction(FPerVertexPaintActionArgs& InArgs, FPerVertexPaintAction Action)
{
	// Get a list of unique vertices indexed by the influenced triangles
	TSet<int32> InfluencedVertices;
	GetPerVertexPaintInfluencedVertices(InArgs, InfluencedVertices);

	if (InfluencedVertices.Num())
	{
		InArgs.Adapter->PreEdit();
		for (const int32 VertexIndex : InfluencedVertices)
		{
		// Apply the action!			
			Action.ExecuteIfBound(InArgs, VertexIndex);
		}
		InArgs.Adapter->PostEdit();
	}

	return (InfluencedVertices.Num() > 0);
}

bool MeshPaintHelpers::ApplyPerTrianglePaintAction(IMeshPaintGeometryAdapter* Adapter, const FVector& CameraPosition, const FVector& HitPosition, const UPaintBrushSettings* Settings, FPerTrianglePaintAction Action)
{
	// Retrieve components world matrix
	const FMatrix& ComponentToWorldMatrix = Adapter->GetComponentToWorldMatrix();

	// Compute the camera position in actor space.  We need this later to check for back facing triangles.
	const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(CameraPosition));
	const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(HitPosition));

	// @todo MeshPaint: Input vector doesn't work well with non-uniform scale
	const float BrushRadius = Settings->GetBrushRadius();
	const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(BrushRadius, 0.0f, 0.0f)).Size();
	const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

	// Get a list of (optionally front-facing) triangles that are within a reasonable distance to the brush
	TArray<uint32> InfluencedTriangles = Adapter->SphereIntersectTriangles(
		ComponentSpaceSquaredBrushRadius,
		ComponentSpaceBrushPosition,
		ComponentSpaceCameraPosition,
		Settings->bOnlyFrontFacingTriangles);

	int32 TriangleIndices[3];

	const TArray<uint32> VertexIndices = Adapter->GetMeshIndices();
	for (uint32 TriangleIndex : InfluencedTriangles)
	{
		// Grab the vertex indices and points for this triangle
		for (int32 TriVertexNum = 0; TriVertexNum < 3; ++TriVertexNum)
		{
			TriangleIndices[TriVertexNum] = VertexIndices[TriangleIndex * 3 + TriVertexNum];
		}

		Action.Execute(Adapter, TriangleIndex, TriangleIndices);
	}

	return (InfluencedTriangles.Num() > 0);
}

struct FPaintedMeshVertex
{
	FVector Position;
	FPackedNormal Normal;
	FColor Color;
};

/** Helper struct for the mesh component vert position octree */
struct FVertexColorPropogationOctreeSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	/**
	 * Get the bounding box of the provided octree element. In this case, the box
	 * is merely the point specified by the element.
	 *
	 * @param	Element	Octree element to get the bounding box for
	 *
	 * @return	Bounding box of the provided octree element
	 */
	FORCEINLINE static FBoxCenterAndExtent GetBoundingBox( const FPaintedMeshVertex& Element )
	{
		return FBoxCenterAndExtent( Element.Position, FVector::ZeroVector );
	}

	/**
	 * Determine if two octree elements are equal
	 *
	 * @param	A	First octree element to check
	 * @param	B	Second octree element to check
	 *
	 * @return	true if both octree elements are equal, false if they are not
	 */
	FORCEINLINE static bool AreElementsEqual( const FPaintedMeshVertex& A, const FPaintedMeshVertex& B )
	{
		return ( A.Position == B.Position && A.Normal == B.Normal && A.Color == B.Color );
	}

	/** Ignored for this implementation */
	FORCEINLINE static void SetElementId( const FPaintedMeshVertex& Element, FOctreeElementId2 Id )
	{
	}
};
typedef TOctree2<FPaintedMeshVertex, FVertexColorPropogationOctreeSemantics> TVertexColorPropogationPosOctree;

void MeshPaintHelpers::ApplyVertexColorsToAllLODs(IMeshPaintGeometryAdapter& GeometryInfo, USkeletalMeshComponent* SkeletalMeshComponent)
{
	checkf(SkeletalMeshComponent != nullptr, TEXT("Invalid Skeletal Mesh Component"));
	USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	if (Mesh)
	{
		FSkeletalMeshRenderData* Resource = Mesh->GetResourceForRendering();
		FSkeletalMeshModel* SrcMesh = Mesh->GetImportedModel();
		if (Resource)
		{
			const int32 NumLODs = Resource->LODRenderData.Num();
			if (NumLODs > 1)
			{
				const FSkeletalMeshLODRenderData& BaseLOD = Resource->LODRenderData[0];
				GeometryInfo.PreEdit();				

				PropagateVertexPaintToAsset(Mesh, 0);

				FBox BaseBounds(ForceInitToZero);

				TArray<FPaintedMeshVertex> PaintedVertices;
				PaintedVertices.Empty(BaseLOD.GetNumVertices());

				FPaintedMeshVertex PaintedVertex;
				for (uint32 VertexIndex = 0; VertexIndex < BaseLOD.GetNumVertices(); ++VertexIndex )
				{
					const FVector VertexPos = (FVector)BaseLOD.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);

					FPackedNormal VertexTangentX, VertexTangentZ;
					VertexTangentX = BaseLOD.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentX(VertexIndex);
					VertexTangentZ = BaseLOD.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex);

					BaseBounds += VertexPos;
					PaintedVertex.Position = VertexPos;
					PaintedVertex.Normal = VertexTangentZ;
					PaintedVertex.Color = BaseLOD.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex);
					PaintedVertices.Add(PaintedVertex);
				}

				for (int32 LODIndex = 1; LODIndex < NumLODs; ++LODIndex)
				{
					// Do something
					FSkeletalMeshLODRenderData& ApplyLOD = Resource->LODRenderData[LODIndex];
					FSkeletalMeshLODModel& SrcLOD = SrcMesh->LODModels[LODIndex];

					FBox CombinedBounds = BaseBounds;
					Mesh->GetLODInfo(LODIndex)->bHasPerLODVertexColors = false;

					if (!ApplyLOD.StaticVertexBuffers.ColorVertexBuffer.IsInitialized())
					{
						ApplyLOD.StaticVertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor::White, ApplyLOD.GetNumVertices());
					}

					for (uint32 VertIndex=0; VertIndex<ApplyLOD.GetNumVertices(); VertIndex++)
					{
						const FVector VertexPos = (FVector)ApplyLOD.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertIndex);
						CombinedBounds += VertexPos;
					}
					
					TVertexColorPropogationPosOctree VertPosOctree(CombinedBounds.GetCenter(), CombinedBounds.GetExtent().GetMax());
					
					// Add each old vertex to the octree
					for (const FPaintedMeshVertex& Vertex : PaintedVertices)
					{
						VertPosOctree.AddElement(Vertex);
					}

					// Iterate over each new vertex position, attempting to find the old vertex it is closest to, applying
					// the color of the old vertex to the new position if possible.
					const float DistanceOverNormalThreshold = KINDA_SMALL_NUMBER;
					check(SrcLOD.NumVertices == ApplyLOD.GetNumVertices());
					for (uint32 VertexIndex = 0; VertexIndex < ApplyLOD.GetNumVertices(); ++VertexIndex)
					{
						TArray<FPaintedMeshVertex> PointsToConsider;
						const FVector CurPosition = (FVector)ApplyLOD.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);

						FPackedNormal VertexTangentZ;
						VertexTangentZ = BaseLOD.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex);
						
						FVector CurNormal = VertexTangentZ.ToFVector();

						// Iterate through the octree attempting to find the vertices closest to the current new point
						VertPosOctree.FindNearbyElements(CurPosition, [&PointsToConsider](const FPaintedMeshVertex& Vertex)
						{
							PointsToConsider.Add(Vertex);
						});

						// If any points to consider were found, iterate over each and find which one is the closest to the new point 
						if (PointsToConsider.Num() > 0)
						{
							int32 BestVertexIndex = 0;
							FVector BestVertexNormal = PointsToConsider[BestVertexIndex].Normal.ToFVector();

							float BestDistanceSquared = (PointsToConsider[BestVertexIndex].Position - CurPosition).SizeSquared();
							float BestNormalDot = BestVertexNormal | CurNormal;

							for (int32 ConsiderationIndex = 1; ConsiderationIndex < PointsToConsider.Num(); ++ConsiderationIndex)
							{
								FPaintedMeshVertex& CheckVertex = PointsToConsider[ConsiderationIndex];
								FVector VertexNormal = CheckVertex.Normal.ToFVector();

								const float DistSqrd = (CheckVertex.Position - CurPosition).SizeSquared();
								const float NormalDot = VertexNormal | CurNormal;
								if (DistSqrd < BestDistanceSquared - DistanceOverNormalThreshold)
								{
									BestVertexIndex = ConsiderationIndex;
									
									BestDistanceSquared = DistSqrd;
									BestNormalDot = NormalDot;
								}
								else if (DistSqrd < BestDistanceSquared + DistanceOverNormalThreshold && NormalDot > BestNormalDot)
								{
									BestVertexIndex = ConsiderationIndex;
									BestDistanceSquared = DistSqrd;
									BestNormalDot = NormalDot;
								}
							}

							ApplyLOD.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = PointsToConsider[BestVertexIndex].Color;
							// Also apply to the skeletal mesh source mesh
							int32 SectionIndex = INDEX_NONE;
							int32 SectionVertexIndex = INDEX_NONE;
							SrcLOD.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);
							SrcLOD.Sections[SectionIndex].SoftVertices[SectionVertexIndex].Color = PointsToConsider[BestVertexIndex].Color;
						}
					}
					PropagateVertexPaintToAsset(Mesh, LODIndex);
				}
				GeometryInfo.PostEdit();
			}
		}
	}
}
