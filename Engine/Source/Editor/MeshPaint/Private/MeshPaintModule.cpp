// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintModule.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Styling/AppStyle.h"
#include "EditorModeRegistry.h"
#include "EditorModes.h"
#include "MeshPainterCommands.h"
#include "MeshPaintAdapterFactory.h"
#include "MeshPaintStaticMeshAdapter.h"
#include "MeshPaintSplineMeshAdapter.h"
#include "MeshPaintSkeletalMeshAdapter.h"

#include "ImportVertexColorOptionsCustomization.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "MeshPaintSettings.h"
#include "ISettingsModule.h"


//////////////////////////////////////////////////////////////////////////
// FMeshPaintModule

namespace MeshPaintModuleConstants
{
	const static FName SettingsContainerName("Editor");
	const static FName SettingsCategoryName("ContentEditors");
	const static FName SettingsSectionName("Mesh Paint");
}

class FMeshPaintModule : public IMeshPaintModule
{
public:
	
	/**
	 * Called right after the module's DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override
	{
		FMeshPainterCommands::Register();

		RegisterGeometryAdapterFactory(MakeShareable(new FMeshPaintGeometryAdapterForSplineMeshesFactory));
		RegisterGeometryAdapterFactory(MakeShareable(new FMeshPaintGeometryAdapterForStaticMeshesFactory));
		RegisterGeometryAdapterFactory(MakeShareable(new FMeshPaintGeometryAdapterForSkeletalMeshesFactory));

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("VertexColorImportOptions", FOnGetDetailCustomizationInstance::CreateStatic(&FVertexColorImportOptionsCustomization::MakeInstance));

		ISettingsModule* SettingsModule = FModuleManager::LoadModulePtr<ISettingsModule>("Settings");
		if(SettingsModule)
		{
			SettingsModule->RegisterSettings(
				MeshPaintModuleConstants::SettingsContainerName, 
				MeshPaintModuleConstants::SettingsCategoryName, 
				MeshPaintModuleConstants::SettingsSectionName,
				NSLOCTEXT("MeshPaintModule", "SettingsName", "Mesh Paint"),
				NSLOCTEXT("MeshPaintModule", "SettingsDesc", "Settings related to mesh painting within the editor."),
				GetMutableDefault<UMeshPaintSettings>());
		}
	}

	virtual void ShutdownModule() override
	{
		FMeshPainterCommands::Unregister();

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("VertexColorImportOptions");

		ISettingsModule* SettingsModule = FModuleManager::LoadModulePtr<ISettingsModule>("Settings");
		if(SettingsModule)
		{
			SettingsModule->UnregisterSettings(
				MeshPaintModuleConstants::SettingsContainerName,
				MeshPaintModuleConstants::SettingsCategoryName,
				MeshPaintModuleConstants::SettingsSectionName);
		}
	}

	virtual void RegisterGeometryAdapterFactory(TSharedRef<IMeshPaintGeometryAdapterFactory> Factory) override
	{
		FMeshPaintAdapterFactory::FactoryList.Add(Factory);
	}

	virtual void UnregisterGeometryAdapterFactory(TSharedRef<IMeshPaintGeometryAdapterFactory> Factory) override
	{
		FMeshPaintAdapterFactory::FactoryList.Remove(Factory);
	}
};

IMPLEMENT_MODULE( FMeshPaintModule, MeshPaint );
