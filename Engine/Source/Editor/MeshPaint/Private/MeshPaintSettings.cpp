// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintSettings.h"

#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPaintSettings)

UPaintBrushSettings::UPaintBrushSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	BrushRadius(128.0f),
	BrushStrength(0.5f),
	BrushFalloffAmount(0.5f),
	bEnableFlow(true),
	bOnlyFrontFacingTriangles(true),
	ColorViewMode(EMeshPaintColorViewMode::Normal)	
{
	const FName ClampMin("ClampMin");
	const FName ClampMax("ClampMax");

	{
		const FProperty* BrushRadiusProperty = UPaintBrushSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, BrushRadius));
		BrushRadiusMin = BrushRadiusProperty->GetFloatMetaData(ClampMin);
		BrushRadiusMax = BrushRadiusProperty->GetFloatMetaData(ClampMax);

		GConfig->GetFloat(TEXT("MeshPaintEdit"), TEXT("DefaultBrushRadius"), BrushRadius, GEditorPerProjectIni);
		BrushRadius = FMath::Clamp(BrushRadius, BrushRadiusMin, BrushRadiusMax);
	}

	{
		const FProperty* BrushStrengthProperty = UPaintBrushSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, BrushStrength));
		float BrushStrengthMin = BrushStrengthProperty->GetFloatMetaData(ClampMin);
		float BrushStrengthMax = BrushStrengthProperty->GetFloatMetaData(ClampMax);

		GConfig->GetFloat(TEXT("MeshPaintEdit"), TEXT("DefaultBrushStrength"), BrushStrength, GEditorPerProjectIni);
		BrushStrength = FMath::Clamp(BrushStrength, BrushStrengthMin, BrushStrengthMax);
	}

	{
		const FProperty* BrushFalloffProperty = UPaintBrushSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, BrushFalloffAmount));
		float BrushFalloffMin = BrushFalloffProperty->GetFloatMetaData(ClampMin);
		float BrushFalloffMax = BrushFalloffProperty->GetFloatMetaData(ClampMax);

		GConfig->GetFloat(TEXT("MeshPaintEdit"), TEXT("DefaultBrushFalloff"), BrushFalloffAmount, GEditorPerProjectIni);
		BrushFalloffAmount = FMath::Clamp(BrushFalloffAmount, BrushFalloffMin, BrushFalloffMax);
	}

	GConfig->GetBool(TEXT("MeshPaintEdit"), TEXT("IgnoreBackFacing"), bOnlyFrontFacingTriangles, GEditorPerProjectIni);

	GConfig->GetBool(TEXT("MeshPaintEdit"), TEXT("EnableBrushFlow"), bEnableFlow, GEditorPerProjectIni);

	FString ColorViewModeString;
	if (GConfig->GetString(TEXT("MeshPaintEdit"), TEXT("ColorViewMode"), ColorViewModeString, GEditorPerProjectIni))
	{
		const UEnum* ColorViewModeEnum = StaticEnum<EMeshPaintColorViewMode>();
		int64 Value = ColorViewModeEnum->GetValueByNameString(ColorViewModeString);
		if (Value != INDEX_NONE)
		{
			ColorViewMode = (EMeshPaintColorViewMode) Value;
		}
	}
}

UPaintBrushSettings::~UPaintBrushSettings()
{
}

void UPaintBrushSettings::SetBrushRadius(float InRadius)
{
	BrushRadius = (float)FMath::Clamp(InRadius, BrushRadiusMin, BrushRadiusMax);
	GConfig->SetFloat(TEXT("MeshPaintEdit"), TEXT("DefaultBrushRadius"), BrushRadius, GEditorPerProjectIni);
}

void UPaintBrushSettings::SetBrushStrength(float InStrength)
{
	BrushStrength = FMath::Clamp(InStrength, 0.f, 1.f);
	GConfig->SetFloat(TEXT("MeshPaintEdit"), TEXT("DefaultBrushStrength"), BrushStrength, GEditorPerProjectIni);
}

void UPaintBrushSettings::SetBrushFalloff(float InFalloff)
{
	BrushFalloffAmount = FMath::Clamp(InFalloff, 0.f, 1.f);
	GConfig->SetFloat(TEXT("MeshPaintEdit"), TEXT("DefaultBrushFalloff"), BrushFalloffAmount, GEditorPerProjectIni);
}

void UPaintBrushSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property == nullptr || PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}

	FName PropertyName = PropertyChangedEvent.Property->GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, BrushRadius))
	{
		GConfig->SetFloat(TEXT("MeshPaintEdit"), TEXT("DefaultBrushRadius"), BrushRadius, GEditorPerProjectIni);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, BrushStrength))
	{
		GConfig->SetFloat(TEXT("MeshPaintEdit"), TEXT("DefaultBrushStrength"), BrushStrength, GEditorPerProjectIni);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, BrushFalloffAmount))
	{
		GConfig->SetFloat(TEXT("MeshPaintEdit"), TEXT("DefaultBrushFalloff"), BrushFalloffAmount, GEditorPerProjectIni);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, bOnlyFrontFacingTriangles))
	{
		GConfig->SetBool(TEXT("MeshPaintEdit"), TEXT("IgnoreBackFacing"), bOnlyFrontFacingTriangles, GEditorPerProjectIni);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, bEnableFlow))
	{
		GConfig->SetBool(TEXT("MeshPaintEdit"), TEXT("EnableBrushFlow"), bEnableFlow, GEditorPerProjectIni);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, ColorViewMode))
	{
		const UEnum* ColorViewModeEnum = StaticEnum<EMeshPaintColorViewMode>();
		FString ColorViewModeString = ColorViewModeEnum->GetNameStringByValue((int64) ColorViewMode);
		GConfig->SetString(TEXT("MeshPaintEdit"), TEXT("ColorViewMode"), *ColorViewModeString, GEditorPerProjectIni);
	}
}
