// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintSkeletalMeshAdapter.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "PhysicsEngine/BodySetup.h"
#include "MeshPaintHelpers.h"
#include "MeshPaintTypes.h"
#include "ComponentReregisterContext.h"
#include "StaticMeshAttributes.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Rendering/RenderCommandPipes.h"

DEFINE_LOG_CATEGORY_STATIC(LogMeshPaintSkeletalMeshAdapter, Log, All);

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshes

FMeshPaintGeometryAdapterForSkeletalMeshes::FMeshToComponentMap FMeshPaintGeometryAdapterForSkeletalMeshes::MeshToComponentMap;


//HACK for 4.24.2 we cannot change public API so we use this global function to remap and propagate the vertex color data to the imported model when the user release the mouse
void PropagateVertexPaintToAsset(USkeletalMesh* SkeletalMesh, int32 LODIndex)
{
	struct FMatchFaceData
	{
		int32 SoftVertexIndexes[3];
	};

	if (!SkeletalMesh || !SkeletalMesh->HasMeshDescription(LODIndex))
	{
		// We do not propagate vertex color for LODs that don't have editable mesh data.
		return;
	}

	auto GetMatchKey = [](const FVector3f& PositionA, const FVector3f& PositionB, const FVector3f& PositionC)->FSHAHash
	{
		FSHA1 SHA;
		FSHAHash SHAHash;

		SHA.Update(reinterpret_cast<const uint8*>(&PositionA), sizeof(FVector3f));
		SHA.Update(reinterpret_cast<const uint8*>(&PositionB), sizeof(FVector3f));
		SHA.Update(reinterpret_cast<const uint8*>(&PositionC), sizeof(FVector3f));
		SHA.Final();
		SHA.GetHash(&SHAHash.Hash[0]);

		return SHAHash;
	};

	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	const TArray<uint32>& SrcIndexBuffer = LODModel.IndexBuffer;

	TArray<FSoftSkinVertex> SrcVertices;
	LODModel.GetVertices(SrcVertices);

	FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);

	TMap<FSHAHash, FMatchFaceData> MatchTriangles;
	MatchTriangles.Reserve(MeshDescription->Triangles().Num());

	for (int32 IndexBufferIndex = 0, SrcIndexBufferNum = SrcIndexBuffer.Num(); IndexBufferIndex < SrcIndexBufferNum; IndexBufferIndex += 3)
	{
		const FVector3f& PositionA = SrcVertices[SrcIndexBuffer[IndexBufferIndex]].Position;
		const FVector3f& PositionB = SrcVertices[SrcIndexBuffer[IndexBufferIndex + 1]].Position;
		const FVector3f& PositionC = SrcVertices[SrcIndexBuffer[IndexBufferIndex + 2]].Position;

		FSHAHash Key = GetMatchKey(PositionA, PositionB, PositionC);
		FMatchFaceData MatchFaceData;
		MatchFaceData.SoftVertexIndexes[0] = SrcIndexBuffer[IndexBufferIndex];
		MatchFaceData.SoftVertexIndexes[1] = SrcIndexBuffer[IndexBufferIndex + 1];
		MatchFaceData.SoftVertexIndexes[2] = SrcIndexBuffer[IndexBufferIndex + 2];
		MatchTriangles.Add(Key, MatchFaceData);
	}

	FStaticMeshAttributes MeshAttributes(*MeshDescription);
	TVertexInstanceAttributesRef<FVector4f> ColorAttribute = MeshAttributes.GetVertexInstanceColors();

	bool bAllVertexFound = true;
	for (const FTriangleID TriangleID: MeshDescription->Triangles().GetElementIDs())
	{
		TArrayView<const FVertexID> TriangleVertexIDs = MeshDescription->GetTriangleVertices(TriangleID);

		FVector3f PositionA = MeshDescription->GetVertexPosition(TriangleVertexIDs[0]);
		FVector3f PositionB = MeshDescription->GetVertexPosition(TriangleVertexIDs[1]);
		FVector3f PositionC = MeshDescription->GetVertexPosition(TriangleVertexIDs[2]);

		const FSHAHash Key = GetMatchKey(PositionA, PositionB, PositionC);
		if (const FMatchFaceData* MatchFaceData = MatchTriangles.Find(Key))
		{
			TArrayView<const FVertexInstanceID> TriangleVertexIndexIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);

			for (int32 Index = 0; Index < 3; Index++)
			{
				FLinearColor Color(SrcVertices[MatchFaceData->SoftVertexIndexes[Index]].Color.ReinterpretAsLinear());
				ColorAttribute.Set(TriangleVertexIndexIDs[Index], Color);
			}
		}
		else if (bAllVertexFound)
		{
			bAllVertexFound = false;
			FString SkeletalMeshName(SkeletalMesh->GetName());
			UE_LOGF(LogMeshPaintSkeletalMeshAdapter, Warning, "Some vertex color data could not be applied to the %ls SkeletalMesh asset.", *SkeletalMeshName);
		}
	}
	
	SkeletalMesh->CommitMeshDescription(LODIndex);
}


bool FMeshPaintGeometryAdapterForSkeletalMeshes::Construct(UMeshComponent* InComponent, int32 InMeshLODIndex)
{
	SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InComponent);
	if (SkeletalMeshComponent != nullptr)
	{
		SkeletalMeshChangedHandle = SkeletalMeshComponent->RegisterOnSkeletalMeshPropertyChanged(USkeletalMeshComponent::FOnSkeletalMeshPropertyChanged::CreateRaw(this, &FMeshPaintGeometryAdapterForSkeletalMeshes::OnSkeletalMeshChanged));

		if (SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr)
		{
			ReferencedSkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
			MeshLODIndex = InMeshLODIndex;
			const bool bSuccess = Initialize();
			return bSuccess;
		}
	}

	return false;
}

FMeshPaintGeometryAdapterForSkeletalMeshes::~FMeshPaintGeometryAdapterForSkeletalMeshes()
{
	if (SkeletalMeshComponent != nullptr)
	{
		SkeletalMeshComponent->UnregisterOnSkeletalMeshPropertyChanged(SkeletalMeshChangedHandle);
	}
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::OnSkeletalMeshChanged()
{
	OnRemoved();
	ReferencedSkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	if (SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr)
	{	
		Initialize();
		OnAdded();
	}	
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::OnPostMeshCached(USkeletalMesh* SkeletalMesh)
{
	if (ReferencedSkeletalMesh == SkeletalMesh)
	{
		OnSkeletalMeshChanged();
	}
}

bool FMeshPaintGeometryAdapterForSkeletalMeshes::Initialize()
{
	check(ReferencedSkeletalMesh == SkeletalMeshComponent->GetSkeletalMeshAsset());

	bool bInitializationResult = false;

	MeshResource = ReferencedSkeletalMesh->GetResourceForRendering();
	if (MeshResource != nullptr)
	{
		LODData = &MeshResource->LODRenderData[MeshLODIndex];
		checkf(ReferencedSkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(MeshLODIndex), TEXT("Invalid Imported Model index for vertex painting"));
		LODModel = &ReferencedSkeletalMesh->GetImportedModel()->LODModels[MeshLODIndex];

		bInitializationResult = FBaseMeshPaintGeometryAdapter::Initialize();
	}

	
	return bInitializationResult;
}

bool FMeshPaintGeometryAdapterForSkeletalMeshes::InitializeVertexData()
{
	// Retrieve mesh vertex and index data 
	const int32 NumVertices = LODData->GetNumVertices();
	MeshVertices.Reset();
	MeshVertices.AddDefaulted(NumVertices);
	for (int32 Index = 0; Index < NumVertices; Index++)
	{
		const FVector3f& Position = LODData->StaticVertexBuffers.PositionVertexBuffer.VertexPosition(Index);
		MeshVertices[Index] = (FVector)Position;
	}

	MeshIndices.Reserve(LODData->MultiSizeIndexContainer.GetIndexBuffer()->Num());
	LODData->MultiSizeIndexContainer.GetIndexBuffer(MeshIndices);

	return (MeshVertices.Num() >= 0 && MeshIndices.Num() > 0);
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::InitializeAdapterGlobals()
{
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
		MeshToComponentMap.Empty();
	}
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::CleanupGlobals()
{
	for (auto& Pair : MeshToComponentMap)
	{
		if (Pair.Key && Pair.Value.RestoreBodySetup)
		{
			Pair.Key->SetBodySetup(Pair.Value.RestoreBodySetup);
		}
	}

	MeshToComponentMap.Empty();
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::OnAdded()
{
	checkf(SkeletalMeshComponent, TEXT("Invalid SkeletalMesh Component"));
	checkf(ReferencedSkeletalMesh, TEXT("Invalid reference to Skeletal Mesh"));
	checkf(ReferencedSkeletalMesh == SkeletalMeshComponent->GetSkeletalMeshAsset(), TEXT("Referenced Skeletal Mesh does not match one in Component"));

	FSkeletalMeshReferencers& SkeletalMeshReferencers = MeshToComponentMap.FindOrAdd(ReferencedSkeletalMesh);

	checkf(!SkeletalMeshReferencers.Referencers.ContainsByPredicate(
		[this](const FSkeletalMeshReferencers::FReferencersInfo& Info)
		{
			return Info.SkeletalMeshComponent == this->SkeletalMeshComponent;
		}), TEXT("This Skeletal Mesh Component has already been Added"));

	// If this is the first attempt to add a temporary body setup to the mesh, do it
	if (SkeletalMeshReferencers.Referencers.Num() == 0)
	{
		// Remember the old body setup (this will be added as a GC reference so that it doesn't get destroyed)
		const USkeletalMesh* ReferencedSkeletalMeshConst = ReferencedSkeletalMesh;
		SkeletalMeshReferencers.RestoreBodySetup = ReferencedSkeletalMeshConst->GetBodySetup();

		if (SkeletalMeshReferencers.RestoreBodySetup)
		{
			// Create a new body setup from the mesh's main body setup. This has to have the skeletal mesh as its outer,
			// otherwise the body instance will not be created correctly.
			UBodySetup* TempBodySetupRaw = DuplicateObject<UBodySetup>(ReferencedSkeletalMeshConst->GetBodySetup(), ReferencedSkeletalMesh);
			TempBodySetupRaw->ClearFlags(RF_Transactional);

			// Set collide all flag so that the body creates physics meshes using ALL elements from the mesh not just the collision mesh.
			TempBodySetupRaw->bMeshCollideAll = true;

			// This forces it to recreate the physics mesh.
			TempBodySetupRaw->InvalidatePhysicsData();

			// Force it to use high detail tri-mesh for collisions.
			TempBodySetupRaw->CollisionTraceFlag = CTF_UseComplexAsSimple;
			TempBodySetupRaw->AggGeom.ConvexElems.Empty();

			// Set as new body setup
			ReferencedSkeletalMesh->SetBodySetup(TempBodySetupRaw);
		}
	}

	SkeletalMeshComponent->bUseRefPoseOnInitAnim = true;
	SkeletalMeshComponent->InitAnim(true);
	ECollisionEnabled::Type CachedCollisionType = SkeletalMeshComponent->BodyInstance.GetCollisionEnabled();
	SkeletalMeshReferencers.Referencers.Emplace(SkeletalMeshComponent, CachedCollisionType);

	// Force the collision type to not be 'NoCollision' without it the line trace will always fail. 
	if (CachedCollisionType == ECollisionEnabled::NoCollision)
	{
		SkeletalMeshComponent->BodyInstance.SetCollisionEnabled(ECollisionEnabled::QueryOnly, false);
	}

	// Set new physics state for the component
	SkeletalMeshComponent->RecreatePhysicsState();

	// Register callback for when the skeletal mesh is cached underneath us
	ReferencedSkeletalMesh->OnPostMeshCached().AddRaw(this, &FMeshPaintGeometryAdapterForSkeletalMeshes::OnPostMeshCached);
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::OnRemoved()
{
	// If the referenced skeletal mesh has been destroyed (and nulled by GC), don't try to do anything more.
	// It should be in the process of removing all global geometry adapters if it gets here in this situation.
	if (!ReferencedSkeletalMesh || !SkeletalMeshComponent)
	{
		return;
	}
	PropagateVertexPaintToAsset(ReferencedSkeletalMesh, MeshLODIndex);

	// Remove a reference from the skeletal mesh map
	FSkeletalMeshReferencers* SkeletalMeshReferencers = MeshToComponentMap.Find(ReferencedSkeletalMesh);
	checkf(SkeletalMeshReferencers, TEXT("Could not find Reference to Skeletal Mesh"));
	checkf(SkeletalMeshReferencers->Referencers.Num() > 0, TEXT("Skeletal Mesh does not have any referencers"));

	const int32 Index = SkeletalMeshReferencers->Referencers.IndexOfByPredicate(
		[this](const FSkeletalMeshReferencers::FReferencersInfo& Info)
		{
			return Info.SkeletalMeshComponent == this->SkeletalMeshComponent;
		}
	);
	check(Index != INDEX_NONE);

	SkeletalMeshComponent->bUseRefPoseOnInitAnim = false;
	SkeletalMeshComponent->InitAnim(true);
	SkeletalMeshComponent->BodyInstance.SetCollisionEnabled(SkeletalMeshReferencers->Referencers[Index].CachedCollisionType, false);
	SkeletalMeshComponent->RecreatePhysicsState();

	SkeletalMeshReferencers->Referencers.RemoveAtSwap(Index);

	// If the last reference was removed, restore the body setup for the static mesh
	if (SkeletalMeshReferencers->Referencers.Num() == 0)
	{
		if (SkeletalMeshReferencers->RestoreBodySetup != nullptr)
		{
			ReferencedSkeletalMesh->SetBodySetup(SkeletalMeshReferencers->RestoreBodySetup);
		}
		
		verify(MeshToComponentMap.Remove(ReferencedSkeletalMesh) == 1);
	}

	ReferencedSkeletalMesh->OnPostMeshCached().RemoveAll(this);
}

bool LegacySegmentTriangleIntersection(const FVector& StartPoint, const FVector& EndPoint, const FVector& A, const FVector& B, const FVector& C, FVector& OutIntersectPoint, FVector& OutTriangleNormal)
{
	const FVector BA = A - B;
	const FVector CB = B - C;
	const FVector TriNormal = BA ^ CB;

	bool bCollide = FMath::SegmentPlaneIntersection(StartPoint, EndPoint, FPlane(A, TriNormal), OutIntersectPoint);
	if (!bCollide)
	{
		return false;
	}

	FVector BaryCentric = FMath::ComputeBaryCentric2D(OutIntersectPoint, A, B, C);
	if (BaryCentric.X > 0.0f && BaryCentric.Y > 0.0f && BaryCentric.Z > 0.0f)
	{
		OutTriangleNormal = TriNormal;
		return true;
	}
	return false;
}
bool FMeshPaintGeometryAdapterForSkeletalMeshes::LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const
{
	const bool bHitBounds = FMath::LineSphereIntersection(Start, End.GetSafeNormal(), (End - Start).SizeSquared(), SkeletalMeshComponent->Bounds.Origin, SkeletalMeshComponent->Bounds.SphereRadius);
	const float SqrRadius = FMath::Square(SkeletalMeshComponent->Bounds.SphereRadius);
	const bool bInsideBounds = (SkeletalMeshComponent->Bounds.ComputeSquaredDistanceFromBoxToPoint(Start) <= SqrRadius) || (SkeletalMeshComponent->Bounds.ComputeSquaredDistanceFromBoxToPoint(End) <= SqrRadius);
	const bool bHitPhysicsBodies = SkeletalMeshComponent->LineTraceComponent(OutHit, Start, End, Params);

	bool bHitTriangle = false;
	if ((bHitBounds || bInsideBounds) && !bHitPhysicsBodies)
	{
		const int32 NumTriangles = MeshIndices.Num() / 3;
		const FTransform& ComponentTransform = SkeletalMeshComponent->GetComponentTransform();
		const FTransform InverseComponentTransform = ComponentTransform.Inverse();
		const FVector LocalStart = InverseComponentTransform.TransformPosition(Start);
		const FVector LocalEnd = InverseComponentTransform.TransformPosition(End);

		float MinDistance = FLT_MAX;
		FVector Intersect;
		FVector Normal;

		for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
		{
			// Compute the normal of the triangle
			const FVector& P0 = MeshVertices[MeshIndices[(TriangleIndex * 3) + 0]];
			const FVector& P1 = MeshVertices[MeshIndices[(TriangleIndex * 3) + 1]];
			const FVector& P2 = MeshVertices[MeshIndices[(TriangleIndex * 3) + 2]];

			const FVector TriNorm = (P1 - P0) ^ (P2 - P0);

			//check collinearity of A,B,C
			if (TriNorm.SizeSquared() > SMALL_NUMBER)
			{
				FVector IntersectPoint;
				FVector HitNormal;
				bool bHit = LegacySegmentTriangleIntersection(LocalStart, LocalEnd, P0, P1, P2, IntersectPoint, HitNormal);

				if (bHit)
				{
					const float Distance = (LocalStart - IntersectPoint).SizeSquared();
					if (Distance < MinDistance)
					{
						MinDistance = Distance;
						Intersect = IntersectPoint;
						Normal = HitNormal;
					}
				}
			}
		}

		if (MinDistance != FLT_MAX)
		{
			OutHit.Component = SkeletalMeshComponent;
			OutHit.Normal = Normal.GetSafeNormal();
			OutHit.Location = ComponentTransform.TransformPosition(Intersect);
			OutHit.bBlockingHit = true;
			bHitTriangle = true;
		}
	}	

	return bHitPhysicsBodies || bHitTriangle;
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{
	DefaultQueryPaintableTextures(MaterialIndex, SkeletalMeshComponent, OutDefaultIndex, InOutTextureList);
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const
{
	DefaultApplyOrRemoveTextureOverride(SkeletalMeshComponent, SourceTexture, OverrideTexture);
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::AddReferencedObjectsGlobals(FReferenceCollector& Collector)
{
	for (auto& Pair : MeshToComponentMap)
	{
		Collector.AddReferencedObject(Pair.Key);
		Collector.AddReferencedObject(Pair.Value.RestoreBodySetup);
		for (FSkeletalMeshReferencers::FReferencersInfo& ReferencerInfo : Pair.Value.Referencers)
		{
			Collector.AddReferencedObject(ReferencerInfo.SkeletalMeshComponent);
		}
	}
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ReferencedSkeletalMesh);
	Collector.AddReferencedObject(SkeletalMeshComponent);
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const
{
	OutTextureCoordinate = FVector2D(LODData->StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, ChannelIndex));
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::PreEdit()
{
	FlushRenderingCommands();

	SkeletalMeshComponent->Modify();

	ReferencedSkeletalMesh->SetFlags(RF_Transactional);
	ReferencedSkeletalMesh->Modify();

	ReferencedSkeletalMesh->SetHasVertexColors(true);
	ReferencedSkeletalMesh->SetVertexColorGuid(FGuid::NewGuid());

	// Release the static mesh's resources.
	ReferencedSkeletalMesh->ReleaseResources();

	// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
	// allocated, and potentially accessing the UStaticMesh.
	ReferencedSkeletalMesh->ReleaseResourcesFence.Wait();

	if (LODData->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() == 0)
	{
		// Mesh doesn't have a color vertex buffer yet!  We'll create one now.
		LODData->StaticVertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor(255, 255, 255, 255), LODData->GetNumVertices());
		ReferencedSkeletalMesh->SetHasVertexColors(true);
		ReferencedSkeletalMesh->SetVertexColorGuid(FGuid::NewGuid());
		BeginInitResource(&LODData->StaticVertexBuffers.ColorVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	}
	//Make sure we change the import data so the re-import do not replace the new data
	if (ReferencedSkeletalMesh->GetAssetImportData())
	{
		UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(ReferencedSkeletalMesh->GetAssetImportData());
		if (ImportData && ImportData->VertexColorImportOption != EVertexColorImportOption::Ignore)
		{
			ImportData->SetFlags(RF_Transactional);
			ImportData->Modify();
			ImportData->VertexColorImportOption = EVertexColorImportOption::Ignore;
		}
	}
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::PostEdit()
{
	TUniquePtr< FSkinnedMeshComponentRecreateRenderStateContext > RecreateRenderStateContext = MakeUnique<FSkinnedMeshComponentRecreateRenderStateContext>(ReferencedSkeletalMesh);
	ReferencedSkeletalMesh->InitResources();
	ReferencedSkeletalMesh->GetOnMeshChanged().Broadcast();
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance /*= true*/) const
{
	if (LODData->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > 0)
	{
		check((int32)LODData->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > VertexIndex);
		OutColor = LODData->StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex);
	}
}

void FMeshPaintGeometryAdapterForSkeletalMeshes::SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance /*= true*/)
{
	if (LODData->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > 0)
	{
		LODData->StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = Color;

		int32 SectionIndex = INDEX_NONE;
		int32 SectionVertexIndex = INDEX_NONE;		
		LODModel->GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);
		LODModel->Sections[SectionIndex].SoftVertices[SectionVertexIndex].Color = Color;

		if (!ReferencedSkeletalMesh->GetLODInfo(MeshLODIndex)->bHasPerLODVertexColors)
		{
			ReferencedSkeletalMesh->GetLODInfo(MeshLODIndex)->bHasPerLODVertexColors = true;
		}
	}
}

FMatrix FMeshPaintGeometryAdapterForSkeletalMeshes::GetComponentToWorldMatrix() const
{
	return SkeletalMeshComponent->GetComponentToWorld().ToMatrixWithScale();
}

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshesFactory

TSharedPtr<IMeshPaintGeometryAdapter> FMeshPaintGeometryAdapterForSkeletalMeshesFactory::Construct(class UMeshComponent* InComponent, int32 InMeshLODIndex) const
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InComponent))
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr)
		{
			TSharedRef<FMeshPaintGeometryAdapterForSkeletalMeshes> Result = MakeShareable(new FMeshPaintGeometryAdapterForSkeletalMeshes());
			if (Result->Construct(InComponent, InMeshLODIndex))
			{
				return Result;
			}
		}
	}

	return nullptr;
}
