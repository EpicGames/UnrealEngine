// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintStaticMeshAdapter.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "PhysicsEngine/BodySetup.h"
#include "MeshPaintHelpers.h"
#include "ComponentReregisterContext.h"
#include "MeshPaintTypes.h"
#include "Engine/StaticMesh.h"

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForStaticMeshes

FMeshPaintGeometryAdapterForStaticMeshes::FMeshToComponentMap FMeshPaintGeometryAdapterForStaticMeshes::MeshToComponentMap;

bool FMeshPaintGeometryAdapterForStaticMeshes::Construct(UMeshComponent* InComponent, int32 InMeshLODIndex)
{
	StaticMeshComponent = Cast<UStaticMeshComponent>(InComponent);
	if (StaticMeshComponent != nullptr)
	{
		StaticMeshComponent->OnStaticMeshChanged().AddRaw(this, &FMeshPaintGeometryAdapterForStaticMeshes::OnStaticMeshChanged);

		if (StaticMeshComponent->GetStaticMesh() != nullptr)
		{
			ReferencedStaticMesh = StaticMeshComponent->GetStaticMesh();
			MeshLODIndex = InMeshLODIndex;
			ReferencedStaticMesh->OnPostMeshBuild().AddRaw(this, &FMeshPaintGeometryAdapterForStaticMeshes::OnPostMeshBuild);
			const bool bSuccess = Initialize();
			return bSuccess;
		}
	}

	return false;
}

FMeshPaintGeometryAdapterForStaticMeshes::~FMeshPaintGeometryAdapterForStaticMeshes()
{
	if (StaticMeshComponent != nullptr)
	{
		if (ReferencedStaticMesh != nullptr)
		{
			ReferencedStaticMesh->OnPostMeshBuild().RemoveAll(this);
		}

		StaticMeshComponent->OnStaticMeshChanged().RemoveAll(this);
	}
}

void FMeshPaintGeometryAdapterForStaticMeshes::OnPostMeshBuild(UStaticMesh* StaticMesh)
{
	check(StaticMesh == ReferencedStaticMesh);
	Initialize();
}

void FMeshPaintGeometryAdapterForStaticMeshes::OnStaticMeshChanged(UStaticMeshComponent* InStaticMeshComponent)
{
	check(StaticMeshComponent == InStaticMeshComponent);
	OnRemoved();
	ReferencedStaticMesh->OnPostMeshBuild().RemoveAll(this);
	ReferencedStaticMesh = InStaticMeshComponent->GetStaticMesh();
	if (ReferencedStaticMesh)
	{
		ReferencedStaticMesh->OnPostMeshBuild().AddRaw(this, &FMeshPaintGeometryAdapterForStaticMeshes::OnPostMeshBuild);
		Initialize();
		OnAdded();
	}
}

bool FMeshPaintGeometryAdapterForStaticMeshes::Initialize()
{
	check(ReferencedStaticMesh == StaticMeshComponent->GetStaticMesh());
	if (MeshLODIndex < ReferencedStaticMesh->GetNumLODs())
	{
		LODModel = &(ReferencedStaticMesh->GetRenderData()->LODResources[MeshLODIndex]);
		return FBaseMeshPaintGeometryAdapter::Initialize();
	}

	return false;
}

bool FMeshPaintGeometryAdapterForStaticMeshes::InitializeVertexData()
{
	// Retrieve mesh vertex and index data 
	const int32 NumVertices = LODModel->VertexBuffers.PositionVertexBuffer.GetNumVertices();
	MeshVertices.Reset();
	MeshVertices.AddDefaulted(NumVertices);
	for (int32 Index = 0; Index < NumVertices; Index++)
	{
		const FVector& Position = (FVector)LODModel->VertexBuffers.PositionVertexBuffer.VertexPosition(Index);
		MeshVertices[Index] = Position;
	}

	const int32 NumIndices = LODModel->IndexBuffer.GetNumIndices();
	MeshIndices.Reset();
	MeshIndices.AddDefaulted(NumIndices);
	const FIndexArrayView ArrayView = LODModel->IndexBuffer.GetArrayView();
	for (int32 Index = 0; Index < NumIndices; Index++)
	{
		MeshIndices[Index] = ArrayView[Index];
	}

	return (MeshVertices.Num() > 0 && MeshIndices.Num() > 0);
}

void FMeshPaintGeometryAdapterForStaticMeshes::PostEdit()
{
	// Lighting does not need to be invalidated when mesh painting
	const bool bUnbuildLighting = false;

	// Recreate all component states using the referenced static mesh
	FStaticMeshComponentRecreateRenderStateContext RecreateRenderStateContext(ReferencedStaticMesh, bUnbuildLighting);
	
	const bool bUsingInstancedVertexColors = true; // Currently we are only painting to instances 
	// Update gpu resource data 
	if (bUsingInstancedVertexColors)
	{
		// We're only changing instanced vertices on this specific mesh component, so we
		// only need to detach our mesh component
		FComponentReregisterContext ComponentReregisterContext(StaticMeshComponent);

		// If LOD is 0, post-edit all LODs. There's currently no way to tell from here
		// if VertexPaintSettings.bPaintOnSpecificLOD is set to true or not.
		const int32 MaxLOD = (MeshLODIndex == 0) ? StaticMeshComponent->LODData.Num() : (MeshLODIndex + 1);
		for (int32 Index = MeshLODIndex; Index < MaxLOD; ++Index)
		{
			BeginInitResource(StaticMeshComponent->LODData[Index].OverrideVertexColors);
		}
	}
	else
	{
		// Reinitialize the static mesh's resources.
		ReferencedStaticMesh->InitResources();
	}
}

void FMeshPaintGeometryAdapterForStaticMeshes::InitializeAdapterGlobals()
{
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
		MeshToComponentMap.Empty();
	}
}

void FMeshPaintGeometryAdapterForStaticMeshes::CleanupGlobals()
{
	for (auto& Pair : MeshToComponentMap)
	{
		if (Pair.Key && Pair.Value.RestoreBodySetup)
		{
			Pair.Key->SetBodySetup(Pair.Value.RestoreBodySetup);
		}
	}

	MeshToComponentMap.Empty();
}

void FMeshPaintGeometryAdapterForStaticMeshes::OnAdded()
{
	check(StaticMeshComponent);
	check(ReferencedStaticMesh);
	check(ReferencedStaticMesh == StaticMeshComponent->GetStaticMesh());

	FStaticMeshReferencers& StaticMeshReferencers = MeshToComponentMap.FindOrAdd(ReferencedStaticMesh);

	check(!StaticMeshReferencers.Referencers.ContainsByPredicate(
		[this](const FStaticMeshReferencers::FReferencersInfo& Info)
	{
		return Info.StaticMeshComponent == this->StaticMeshComponent;
	}
	));

	bool bBodyChanged = false;

	// If this is the first attempt to add a temporary body setup to the mesh, do it
	if (StaticMeshReferencers.Referencers.Num() == 0)
	{
		// Remember the old body setup (this will be added as a GC reference so that it doesn't get destroyed)
		StaticMeshReferencers.RestoreBodySetup = ReferencedStaticMesh->GetBodySetup();

		// Create a new body setup from the mesh's main body setup. This has to have the static mesh as its outer,
		// otherwise the body instance will not be created correctly.
		UBodySetup* TempBodySetupRaw = DuplicateObject<UBodySetup>(ReferencedStaticMesh->GetBodySetup(), ReferencedStaticMesh);
		TempBodySetupRaw->ClearFlags(RF_Transactional);

		// Set collide all flag so that the body creates physics meshes using ALL elements from the mesh not just the collision mesh.
		TempBodySetupRaw->bMeshCollideAll = true;

		// This forces it to recreate the physics mesh.
		TempBodySetupRaw->InvalidatePhysicsData();

		// Force it to use high detail tri-mesh for collisions.
		TempBodySetupRaw->CollisionTraceFlag = CTF_UseComplexAsSimple;
		TempBodySetupRaw->AggGeom.ConvexElems.Empty();

		// Set as new body setup
		ReferencedStaticMesh->SetBodySetup(TempBodySetupRaw);

		bBodyChanged = true;
	}

	ECollisionEnabled::Type CachedCollisionType = StaticMeshComponent->BodyInstance.GetCollisionEnabled();
	StaticMeshReferencers.Referencers.Emplace(StaticMeshComponent, CachedCollisionType);

	// Force the collision type to not be 'NoCollision' without it the line trace will always fail. 
	if (CachedCollisionType == ECollisionEnabled::NoCollision)
	{
		StaticMeshComponent->BodyInstance.SetCollisionEnabled(ECollisionEnabled::QueryOnly, false);
	}

	if (bBodyChanged)
	{
		// Set new physics state for the component
		StaticMeshComponent->RecreatePhysicsState();
	}
}

void FMeshPaintGeometryAdapterForStaticMeshes::OnRemoved()
{
	// If the referenced static mesh has been destroyed (and nulled by GC), don't try to do anything more.
	// It should be in the process of removing all global geometry adapters if it gets here in this situation.
	if (!ReferencedStaticMesh || !StaticMeshComponent)
	{
		return;
	}

	// Remove a reference from the static mesh map
	FStaticMeshReferencers* StaticMeshReferencers = MeshToComponentMap.Find(ReferencedStaticMesh);

	if (StaticMeshReferencers)
	{
		check(StaticMeshReferencers->Referencers.Num() > 0);
		int32 Index = StaticMeshReferencers->Referencers.IndexOfByPredicate(
			[this](const FStaticMeshReferencers::FReferencersInfo& Info)
		{
			return Info.StaticMeshComponent == this->StaticMeshComponent;
		}
		);

		if(Index != INDEX_NONE)
		{
			StaticMeshComponent->BodyInstance.SetCollisionEnabled(StaticMeshReferencers->Referencers[Index].CachedCollisionType, false);
			StaticMeshComponent->RecreatePhysicsState();

			StaticMeshReferencers->Referencers.RemoveAtSwap(Index);
		}
		else
		{
			// Might be null components.  Remove them
			StaticMeshReferencers->Referencers.RemoveAll([=](const FStaticMeshReferencers::FReferencersInfo& Info)
			{
				return Info.StaticMeshComponent == nullptr;
			});
		}

		// If the last reference was removed, restore the body setup for the static mesh
		if (StaticMeshReferencers->Referencers.Num() == 0)
		{
			ReferencedStaticMesh->SetBodySetup(StaticMeshReferencers->RestoreBodySetup);
			verify(MeshToComponentMap.Remove(ReferencedStaticMesh) == 1);
		}
	}
}

bool FMeshPaintGeometryAdapterForStaticMeshes::LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const
{
	// Ray trace
	return StaticMeshComponent->LineTraceComponent(OutHit, Start, End, Params);
}

void FMeshPaintGeometryAdapterForStaticMeshes::QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{
	DefaultQueryPaintableTextures(MaterialIndex, StaticMeshComponent, OutDefaultIndex, InOutTextureList);
}

void FMeshPaintGeometryAdapterForStaticMeshes::ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const
{
	DefaultApplyOrRemoveTextureOverride(StaticMeshComponent, SourceTexture, OverrideTexture);
}

void FMeshPaintGeometryAdapterForStaticMeshes::AddReferencedObjectsGlobals(FReferenceCollector& Collector)
{
	for (auto& Pair : MeshToComponentMap)
	{
		Collector.AddReferencedObject(Pair.Key);
		Collector.AddReferencedObject(Pair.Value.RestoreBodySetup);
		for (FStaticMeshReferencers::FReferencersInfo& ReferencerInfo : Pair.Value.Referencers)
		{
			Collector.AddReferencedObject(ReferencerInfo.StaticMeshComponent);
		}
	}
}

void FMeshPaintGeometryAdapterForStaticMeshes::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ReferencedStaticMesh);
	Collector.AddReferencedObject(StaticMeshComponent);
}

void FMeshPaintGeometryAdapterForStaticMeshes::GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance /*= true*/) const
{
	if (bInstance)
	{
		FStaticMeshComponentLODInfo* InstanceMeshLODInfo = &StaticMeshComponent->LODData[MeshLODIndex];
		if (!bInstance && LODModel->VertexBuffers.ColorVertexBuffer.GetNumVertices() == 0)
		{
			// Mesh doesn't have a color vertex buffer yet!  We'll create one now.
			LODModel->VertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor(255, 255, 255, 255), LODModel->GetNumVertices());

			// @todo MeshPaint: Make sure this is the best place to do this
			BeginInitResource(&LODModel->VertexBuffers.ColorVertexBuffer);
		}

		// Actor mesh component LOD
		const bool bValidInstanceData = InstanceMeshLODInfo
			&& InstanceMeshLODInfo->OverrideVertexColors
			&& InstanceMeshLODInfo->OverrideVertexColors->GetNumVertices() == LODModel->GetNumVertices();
		if (bValidInstanceData)
		{
			OutColor = InstanceMeshLODInfo->OverrideVertexColors->VertexColor(VertexIndex);
		}
	}
	else
	{
		// Static mesh LOD
		const bool bValidMeshData = LODModel->VertexBuffers.ColorVertexBuffer.GetNumVertices() > (uint32)VertexIndex;
		if (bValidMeshData)
		{
			OutColor = LODModel->VertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex);
		}
	}
}

void FMeshPaintGeometryAdapterForStaticMeshes::SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance /*= true*/)
{
	// Update the mesh!				
	if (bInstance)
	{
		FStaticMeshComponentLODInfo* InstanceMeshLODInfo = &StaticMeshComponent->LODData[MeshLODIndex];
		const bool bValidInstanceData = InstanceMeshLODInfo
			&& InstanceMeshLODInfo->OverrideVertexColors
			&& InstanceMeshLODInfo->OverrideVertexColors->GetNumVertices() == LODModel->GetNumVertices();

		// If there is valid instance data update the color value
		if (bValidInstanceData)
		{
			check(InstanceMeshLODInfo->OverrideVertexColors);
			check((uint32)VertexIndex < InstanceMeshLODInfo->OverrideVertexColors->GetNumVertices());
			check(InstanceMeshLODInfo->OverrideVertexColors->GetNumVertices() == InstanceMeshLODInfo->PaintedVertices.Num());
			InstanceMeshLODInfo->OverrideVertexColors->VertexColor(VertexIndex) = Color;
			InstanceMeshLODInfo->PaintedVertices[VertexIndex].Color = Color;

			// If set on LOD level > 0 means we have per LOD painted vertex color data
			if (MeshLODIndex > 0)
			{
				StaticMeshComponent->bCustomOverrideVertexColorPerLOD = true;
			}
		}
	}	
	else
	{
		const bool bValidMeshData = LODModel->VertexBuffers.ColorVertexBuffer.GetNumVertices() >(uint32)VertexIndex;
		if (bValidMeshData)
		{
			LODModel->VertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = Color;
		}
	}
}

FMatrix FMeshPaintGeometryAdapterForStaticMeshes::GetComponentToWorldMatrix() const
{
	return StaticMeshComponent->GetComponentToWorld().ToMatrixWithScale();
}

void FMeshPaintGeometryAdapterForStaticMeshes::GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const
{
	OutTextureCoordinate = FVector2D(LODModel->VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, ChannelIndex));
}

void FMeshPaintGeometryAdapterForStaticMeshes::PreEdit()
{
	const bool bUsingInstancedVertexColors = true; // Currently we are only painting to instances 
	UStaticMesh* StaticMesh = ReferencedStaticMesh;
	if (bUsingInstancedVertexColors)
	{
		// Mark the mesh component as modified
		StaticMeshComponent->SetFlags(RF_Transactional);
		StaticMeshComponent->Modify();
		StaticMeshComponent->bCustomOverrideVertexColorPerLOD = (MeshLODIndex > 0);
				
		const int32 NumLODs = StaticMesh->GetNumLODs();
		const int32 MaxIndex = (MeshLODIndex == 0) ? NumLODs : (MeshLODIndex + 1);
		// Ensure LODData has enough entries in it, free not required.

		StaticMeshComponent->SetLODDataCount(NumLODs, NumLODs);

		// If LOD is 0, pre-edit all LODs. There's currently no way to tell from here
		// if VertexPaintSettings.bPaintOnSpecificLOD is set to true or not.
		for (int32 Index = MeshLODIndex; Index < MaxIndex; ++Index)
		{
			FStaticMeshComponentLODInfo& InstanceMeshLODInfo = StaticMeshComponent->LODData[Index];
			FStaticMeshLODResources& LODResource = StaticMesh->GetRenderData()->LODResources[Index];

			// Destroy the instance vertex  color array if it doesn't fit
			if (InstanceMeshLODInfo.OverrideVertexColors
				&& InstanceMeshLODInfo.OverrideVertexColors->GetNumVertices() != LODResource.GetNumVertices())
			{
				InstanceMeshLODInfo.ReleaseOverrideVertexColorsAndBlock();
			}

			if (InstanceMeshLODInfo.OverrideVertexColors)
			{
				// Destroy the cached paint data every paint. Painting redefines the source data.
				InstanceMeshLODInfo.PaintedVertices.Empty();
				InstanceMeshLODInfo.BeginReleaseOverrideVertexColors();
				FlushRenderingCommands();
			}
			else
			{
				// Setup the instance vertex color array if we don't have one yet
				InstanceMeshLODInfo.OverrideVertexColors = new FColorVertexBuffer;

				if ((int32)LODResource.VertexBuffers.ColorVertexBuffer.GetNumVertices() >= LODResource.GetNumVertices())
				{
					// copy mesh vertex colors to the instance ones
					InstanceMeshLODInfo.OverrideVertexColors->InitFromColorArray(&LODResource.VertexBuffers.ColorVertexBuffer.VertexColor(0), LODResource.GetNumVertices());
				}
				else
				{
					// Original mesh didn't have any colors, so just use a default color
					InstanceMeshLODInfo.OverrideVertexColors->InitFromSingleColor(FColor::White, LODResource.GetNumVertices());
				}

			}
		}
		// See if the component has to cache its mesh vertex positions associated with override colors
		StaticMeshComponent->CachePaintedDataIfNecessary();
		StaticMeshComponent->StaticMeshDerivedDataKey = StaticMesh->GetRenderData()->DerivedDataKey;
	}
	else
	{
		// Dirty the mesh
		StaticMesh->SetFlags(RF_Transactional);
		StaticMesh->Modify();

		// Release the static mesh's resources.
		StaticMesh->ReleaseResources();

		// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
		// allocated, and potentially accessing the UStaticMesh.
		StaticMesh->ReleaseResourcesFence.Wait();
	}
}

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForStaticMeshesFactory

TSharedPtr<IMeshPaintGeometryAdapter> FMeshPaintGeometryAdapterForStaticMeshesFactory::Construct(class UMeshComponent* InComponent, int32 InMeshLODIndex) const
{
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(InComponent))
	{
		if (StaticMeshComponent->GetStaticMesh() != nullptr)
		{
			TSharedRef<FMeshPaintGeometryAdapterForStaticMeshes> Result = MakeShareable(new FMeshPaintGeometryAdapterForStaticMeshes());
			if (Result->Construct(InComponent, InMeshLODIndex))
			{
				return Result;
			}
		}
	}

	return nullptr;
}
