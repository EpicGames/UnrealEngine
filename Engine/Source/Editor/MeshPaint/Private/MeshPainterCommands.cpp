// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPainterCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/PlatformCrt.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "MeshPainterCommands"

FMeshPainterCommands::FMeshPainterCommands() 
	: TCommands<FMeshPainterCommands>(
		"MeshPainter", 
		NSLOCTEXT("Contexts", "MeshPainter", "Mesh Painter"), 
		NAME_None, 
		FAppStyle::GetAppStyleSetName())
{
}

void FMeshPainterCommands::RegisterCommands()
{
	UI_COMMAND(IncreaseBrushRadius, "Increase Brush Radius", "Press this key to increase brush radius by a percentage of its current size.", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket));
	Commands.Add(IncreaseBrushRadius);
	UI_COMMAND(DecreaseBrushRadius, "Decrease Brush Size", "Press this key to decrease brush radius by a percentage of its current size.", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket));
	Commands.Add(DecreaseBrushRadius);

	UI_COMMAND(IncreaseBrushStrength, "Increase Brush Strength", "Press this key to increase brush strength by a fixed increment.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::RightBracket));
	Commands.Add(IncreaseBrushStrength);
	UI_COMMAND(DecreaseBrushStrength, "Decrease Brush Strength", "Press this key to decrease brush strength by a fixed increment.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::LeftBracket));
	Commands.Add(DecreaseBrushStrength);

	UI_COMMAND(IncreaseBrushFalloff, "Increase Brush Falloff", "Press this key to increase brush falloff by a fixed increment.", EUserInterfaceActionType::Button, FInputChord(FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::RightBracket)));
	Commands.Add(IncreaseBrushFalloff);
	UI_COMMAND(DecreaseBrushFalloff, "Decrease Brush Falloff", "Press this key to decrease brush falloff by a fixed increment.", EUserInterfaceActionType::Button, FInputChord(FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::LeftBracket)));
	Commands.Add(DecreaseBrushFalloff);
}

#undef LOCTEXT_NAMESPACE

