// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "IMeshPaintGeometryAdapter.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "TMeshPaintOctree.h"
#include "Templates/UniquePtr.h"

#define UE_API MESHPAINT_API

/** Base mesh paint geometry adapter, handles basic sphere intersection using a Octree */
class FBaseMeshPaintGeometryAdapter : public IMeshPaintGeometryAdapter
{
public:
	/** Start IMeshPaintGeometryAdapter Overrides */
	UE_API virtual bool Initialize() override;
	UE_API virtual const TArray<FVector>& GetMeshVertices() const override;
	UE_API virtual const TArray<uint32>& GetMeshIndices() const override;
	UE_API virtual void GetVertexPosition(int32 VertexIndex, FVector& OutVertex) const override;
	UE_API virtual TArray<uint32> SphereIntersectTriangles(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const override;
	UE_API virtual void GetInfluencedVertexIndices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TSet<int32> &InfluencedVertices) const override;
	UE_API virtual void GetInfluencedVertexData(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TArray<TPair<int32, FVector>>& OutData) const override;
	UE_API virtual TArray<FVector> SphereIntersectVertices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const override;
	/** End IMeshPaintGeometryAdapter Overrides */

	virtual bool InitializeVertexData() = 0;
protected:
	UE_API bool BuildOctree();
protected:
	/** Index and Vertex data populated by derived classes in InitializeVertexData */
	TArray<FVector> MeshVertices;
	TArray<uint32> MeshIndices;
	/** Octree used for reducing the cost of sphere intersecting with triangles / vertices */
	TUniquePtr<FMeshPaintTriangleOctree> MeshTriOctree;
};

#undef UE_API
