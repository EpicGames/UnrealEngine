// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API MESHPAINT_API

class FReferenceCollector;
class UMeshComponent;
class UTexture;

/**
 * Interface for a class to provide mesh painting support for a subclass of UMeshComponent
 */
class IMeshPaintGeometryAdapter
{
public:
	/** Constructs the adapter for a specific LOD index using the mesh component */
	virtual bool Construct(UMeshComponent* InComponent, int32 MeshLODIndex) = 0;

	/** D-tor */
	virtual ~IMeshPaintGeometryAdapter() = default;
	
	/** Returns whether or not initialization of necessary data was successful */
	virtual bool Initialize() = 0;

	/** Called when this adapter is created and added to its owner (used for setting up) */
	virtual void OnAdded() = 0;
	
	/** Called when this adapter is removed from its owner and deleted (used for cleaning up) */
	virtual void OnRemoved() = 0;

	/** Returns whether or not this adapter is in a valid state */
	virtual bool IsValid() const = 0;	
	
	/** Whether or not this adapter supports texture painting */
	virtual bool SupportsTexturePaint() const = 0;

	/** Whether or not this adapter supports vertex painting */
	virtual bool SupportsVertexPaint() const = 0;

	/** Returns the result of tracing a line on the component represented by this adapter */
	virtual bool LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const = 0;	
	
	/** Retrieves a list of textures which are suitable for painting */
	virtual void QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList) = 0;

	/** Applies or removes an override texture to use while rendering the meshes materials */
	virtual void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const = 0;
	
	/** Reference collecting to prevent GC-ing */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) = 0;
			
	/** Returns the vertices for the current LOD index in the Mesh */
	virtual const TArray<FVector>& GetMeshVertices() const = 0;

	/** Returns the indices for the current LOD index in the Mesh */
	virtual const TArray<uint32>& GetMeshIndices() const = 0;

	/** Returns the Vertex Position at Vertex Index from the Mesh */
	virtual void GetVertexPosition(int32 VertexIndex, FVector& OutVertex) const = 0;

	/** Sets the Vertex Color at Vertex Index to Color inside of the Mesh */
	virtual void SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance = true) = 0;
	
	/** Returns the Texture Coordinate at Vertex Index from the Mesh */
	virtual void GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const = 0;

	/** Returns the Vertex Color at Vertex Index from the Mesh */
	virtual void GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance = true) const = 0;

	/** Returns the Component to World matrix from the Mesh Component */
	virtual FMatrix GetComponentToWorldMatrix() const = 0;

	/** Pre Edit to setup necessary data */
	virtual void PreEdit() = 0;

	/** Post Edit to clear up or update necessary data */
	virtual void PostEdit() = 0;

	/** Default functionality for applying or remove a texture override */
	static UE_API void DefaultApplyOrRemoveTextureOverride(UMeshComponent* InMeshComponent, UTexture* SourceTexture, UTexture* OverrideTexture);

	/** Default functionality for querying textures from a mesh component which are suitable for texture painting */
	static UE_API void DefaultQueryPaintableTextures(int32 MaterialIndex, const UMeshComponent* MeshComponent, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList);

	/** Returns the triangle indices which intersect with the given sphere */
	virtual TArray<uint32> SphereIntersectTriangles(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const = 0;

	/** Retrieves the influenced vertex indices which intersect the given sphere */
	virtual void GetInfluencedVertexIndices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TSet<int32> &InfluencedVertices) const = 0;
	
	/** Retrieves the influenced vertex indices and positions that intersect the given sphere */
	virtual void GetInfluencedVertexData(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TArray<TPair<int32, FVector>>& OutData) const = 0;

	/** Returns the vertex positions which intersect the given sphere */
	virtual TArray<FVector> SphereIntersectVertices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const = 0;
};

#undef UE_API
