// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IMeshPaintGeometryAdapter;
class UMeshComponent;
class FReferenceCollector;

/**
 * Factory for IMeshPaintGeometryAdapter
 */
class IMeshPaintGeometryAdapterFactory
{
public:
	virtual TSharedPtr<IMeshPaintGeometryAdapter> Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) const = 0;
	virtual void InitializeAdapterGlobals() = 0;
	virtual void AddReferencedObjectsGlobals(FReferenceCollector& Collector) = 0;
	virtual void CleanupGlobals() = 0;
	virtual ~IMeshPaintGeometryAdapterFactory() = default;
};
