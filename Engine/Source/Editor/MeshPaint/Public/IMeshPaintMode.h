// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "EdMode.h"

#define UE_API MESHPAINT_API

class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FObjectPostSaveContext;
class FObjectPreSaveContext;
class FSceneView;
class FViewport;
class UFactory;
class IMeshPainter;
class UViewportInteractor;
struct FAssetData;

/**
 * Mesh Paint editor mode
 */
class IMeshPaintEdMode : public FEdMode
{
public:
	/** Constructor */
	UE_API IMeshPaintEdMode();

	/** Destructor */
	UE_API virtual ~IMeshPaintEdMode();

	virtual void Initialize() = 0;
	virtual TSharedPtr<class FModeToolkit> GetToolkit() = 0;

	/** FGCObject interface */
	UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return "IMeshPaintEdMode";
	}

	// FEdMode interface
	virtual bool UsesToolkits() const override { return true; }
	UE_API virtual void Enter() override;
	UE_API virtual void Exit() override;
	UE_API virtual bool ProcessCapturedMouseMoves(FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves) override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override { return true; }
	UE_API virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	UE_API virtual bool InputKey( FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent ) override;
	UE_API virtual void PostUndo() override;
	UE_API virtual void Render( const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI ) override;
	UE_API virtual bool Select( AActor* InActor, bool bInSelected ) override;
	UE_API virtual void ActorSelectionChangeNotify() override;
	virtual bool AllowWidgetMove() override { return false; }
	virtual bool ShouldDrawWidget() const override { return false; }
	virtual bool UsesTransformWidget() const override { return false; }
	UE_API virtual void Tick( FEditorViewportClient* ViewportClient, float DeltaTime ) override;
	UE_API virtual bool ProcessEditDelete() override;
	// End of FEdMode interface

	/** Returns the mesh painter for this mode */
	UE_API IMeshPainter* GetMeshPainter();
private:
	/** Called prior to saving a level */
	UE_API void OnPreSaveWorld(UWorld* World, FObjectPreSaveContext ObjectSaveContext);

	/** Called after saving a level */
	UE_API void OnPostSaveWorld(UWorld* World, FObjectPostSaveContext ObjectSaveContext);

	/** Called when an asset has just been imported */
	UE_API void OnPostImportAsset(UFactory* Factory, UObject* Object);

	/** Called when an asset has just been reimported */
	UE_API void OnPostReimportAsset(UObject* Object, bool bSuccess);

	/** Called when an asset is deleted */
	UE_API void OnAssetRemoved(const FAssetData& AssetData);

	/** Called when the user presses a button on their motion controller device */
	UE_API void OnVRAction( FEditorViewportClient& ViewportClient, class UViewportInteractor* Interactor,
	const struct FViewportActionKeyInput& Action, bool& bOutIsInputCaptured, bool& bWasHandled );

	/** Called when rerunning a construction script causes objects to be replaced */
	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	UE_API void OnResetViewMode();
private:	
	/** When painting in VR, this is the hand index that we're painting with.  Otherwise INDEX_NONE. */
	UViewportInteractor* PaintingWithInteractorInVR;
	
	/** Will store the state of selection locks on start of paint mode so that it can be restored on close */
	bool bWasSelectionLockedOnStart;

	/** Delegate handle for registered selection change lambda */
	FDelegateHandle SelectionChangedHandle;
protected:
	/** Painter used by this edit mode for applying paint actions */
	IMeshPainter* MeshPainter;	
};

#undef UE_API
