// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "InputCoreTypes.h"

#define UE_API MESHPAINT_API

class UMeshPaintSettings;
class UPaintBrushSettings;
class UMeshComponent;
class IMeshPaintGeometryAdapter;
class FScopedTransaction;
class FUICommandList;
class FViewport;
class FPrimitiveDrawInterface;
class FEditorViewportClient;
class FSceneView;
class AActor;

struct FHitResult;

enum class EMeshPaintAction : uint8;

/** Base class for creating a mesh painter, has basic functionality combined with IMeshPaintMode and requires painting/situation-specific information to do actual painting */
class IMeshPainter
{
public:
	UE_API IMeshPainter();
	UE_API virtual ~IMeshPainter();

	/** Renders ray widgets for the active viewport interactors */
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) = 0;

	/** Base ticking functionality, keeps track of time and caches VR interactors */
	UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime);

	/** Returns whether or not painting (of any sorts) is in progress*/
	virtual bool IsPainting() { return bArePainting; }

	/** Registers a base set of commands for altering UBrushSettings */
	UE_API virtual void RegisterCommands(TSharedRef<FUICommandList> CommandList);

	/** Unregisters a base set of commands */
	UE_API virtual void UnregisterCommands(TSharedRef<FUICommandList> CommandList);

	/** 'Normal' painting functionality, called when the user tries to paint on a mesh using the mouse */
	UE_API virtual bool Paint(FViewport* Viewport, const FVector& InCameraOrigin, const FVector& InRayOrigin, const FVector& InRayDirection);

	/** 'Normal' painting functionality, called when the user tries to paint on a mesh using the mouse */
	UE_API virtual bool Paint(FViewport* Viewport, const FVector& InCameraOrigin, const TArrayView<TPair<FVector, FVector>>& Rays);

	/** Allows painter to act on specific key actions */
	UE_API virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent);

	/** If painting is in progress will propagate changes and cleanup the painting state */
	UE_API virtual void FinishPainting();
	
	/** Is called by the owning EdMode when an actor is de-selected in the viewport */
	virtual void ActorSelected(AActor* Actor) = 0;
	/** Is called by the owning EdMode when an actor is selected in the viewport */
	virtual void ActorDeselected(AActor* Actor) = 0;

	/** Tries to retrieves a valid mesh adapter for the given component (derived painters can cache these hence no base implementation) */
	virtual TSharedPtr<IMeshPaintGeometryAdapter> GetMeshAdapterForComponent( const UMeshComponent* Component) = 0;

	/** Functionality to collect painter referenced objects to prevent GC */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) = 0;

	/** Returns an instance of UPaintBrushSettings (base or derived) which contains the basic settings used for painting */
	virtual UPaintBrushSettings* GetBrushSettings() = 0;

	/** Returns an instance of UMeshPainterSettings used for specific types of painting */
	virtual UMeshPaintSettings* GetPainterSettings() = 0;

	/** Returns a SWidget which represents the painters state and allows for changes / extra functionality */
	virtual TSharedPtr<class SWidget> GetWidget() = 0;	

	/** Returns a HitResult structure given a origin and direction to trace a ray for */
	virtual const FHitResult GetHitResult(const FVector& Origin, const FVector& Direction) = 0;

	/** Refreshes the underlying data */
	virtual void Refresh() = 0;

	/** Resets the state of the painter */
	virtual void Reset() = 0;
protected:
	/** Internal painting functionality, is called from Paint and PaintVR and implemented in the derived painters */
	virtual bool PaintInternal(const FVector& InCameraOrigin, const TArrayView<TPair<FVector, FVector>>& Rays, EMeshPaintAction PaintAction, float PaintStrength) = 0;

	/** Renders viewport interactor widget */
	UE_API void RenderInteractors(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI, bool bRenderVertices, ESceneDepthPriorityGroup DepthGroup = SDPG_World);

	/** Functionality for rendering the interactor widget given its positional information and paint action */
	UE_API virtual void RenderInteractorWidget(const FVector& InCameraOrigin, const FVector& InRayOrigin, const FVector& InRayDirection, FPrimitiveDrawInterface* PDI, EMeshPaintAction PaintAction, bool bRenderVertices, ESceneDepthPriorityGroup DepthGroup = SDPG_World);

protected:
	/** Flag for whether or not we are currently painting */
	bool bArePainting;
	/** Time kept since the user has started painting */
	float TimeSinceStartedPainting;
	/** Overall time value kept for drawing effects */
	float Time;

	float WidgetLineThickness;
	FLinearColor WidgetLineColor;
	FLinearColor VertexPointColor;
	FLinearColor HoverVertexPointColor;

	TSharedPtr<FUICommandList> UICommandList;

protected:
	/** Begins and keeps track of an Editor transaction with the given Description */
	UE_API void BeginTransaction(const FText Description);
	/** Ends the current transaction */
	UE_API void EndTransaction();
	/** Painting transaction instance which is currently active */
	FScopedTransaction* PaintTransaction;

private:

	/** Change the brush radius by a percentage, multiplied by the given value. */
	UE_API void ChangeBrushRadius(float Multiplier);

	/** Change the brush radius by one increment, multiplied by the given value */
	UE_API void ChangeBrushStrength(float Multiplier);

	/** Change the brush radius by one increment, multiplied by the given value */
	UE_API void ChangeBrushFalloff(float Multiplier);
};

#undef UE_API
