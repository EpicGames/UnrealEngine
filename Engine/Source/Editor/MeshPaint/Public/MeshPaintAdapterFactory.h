// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "MeshPaintModule.h"
#include "Templates/SharedPointer.h"

#define UE_API MESHPAINT_API

class FReferenceCollector;
class IMeshPaintGeometryAdapterFactory;
class UMeshComponent;

class FMeshPaintAdapterFactory
{
public:
	static UE_API TArray<TSharedPtr<IMeshPaintGeometryAdapterFactory>> FactoryList;

public:
	static UE_API TSharedPtr<class IMeshPaintGeometryAdapter> CreateAdapterForMesh(UMeshComponent* InComponent, int32 InPaintingMeshLODIndex);
	static UE_API void InitializeAdapterGlobals();
	static UE_API void AddReferencedObjectsGlobals(FReferenceCollector& Collector);
	static UE_API void CleanupGlobals();
};

#undef UE_API
