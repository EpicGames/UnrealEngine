// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Engine/HitResult.h"
#include "MeshPaintTypes.h"

#define UE_API MESHPAINT_API

class FMeshPaintParameters;
class UVertexColorImportOptions;
class UTexture2D;
class UStaticMeshComponent;
class UStaticMesh;
class USkeletalMesh;
class IMeshPaintGeometryAdapter;
class UPaintBrushSettings;
class FEditorViewportClient;
class UMeshComponent;
class USkeletalMeshComponent;
class UViewportInteractor;
class FViewport;
class FPrimitiveDrawInterface;
class FSceneView;

struct FStaticMeshComponentLODInfo;

enum class EMeshPaintColorViewMode : uint8;

/** Parameters for paint actions, stored together for convenience */
struct FPerVertexPaintActionArgs
{
	IMeshPaintGeometryAdapter* Adapter;
	FVector CameraPosition;
	FHitResult HitResult;
	const UPaintBrushSettings* BrushSettings;
	EMeshPaintAction Action;
};

/** Delegates used to call per-vertex/triangle actions */
DECLARE_DELEGATE_TwoParams(FPerVertexPaintAction, FPerVertexPaintActionArgs& /*Args*/, int32 /*VertexIndex*/);
DECLARE_DELEGATE_ThreeParams(FPerTrianglePaintAction, IMeshPaintGeometryAdapter* /*Adapter*/, int32 /*TriangleIndex*/, const int32[3] /*Vertex Indices*/);

class MeshPaintHelpers
{
public:
	/** Removes vertex colors associated with the object */
	static UE_API void RemoveInstanceVertexColors(UObject* Obj);

	/** Removes vertex colors associated with the static mesh component */
	static UE_API void RemoveComponentInstanceVertexColors(UStaticMeshComponent* StaticMeshComponent);
	
	/** Propagates per-instance vertex colors to the underlying Static Mesh for the given LOD Index */
	static UE_API bool PropagateColorsToRawMesh(UStaticMesh* StaticMesh, int32 LODIndex, FStaticMeshComponentLODInfo& ComponentLODInfo);	

	/** Retrieves the Vertex Color buffer size for the given LOD level in the Static Mesh */
	static UE_API uint32 GetVertexColorBufferSize(UMeshComponent* MeshComponent, int32 LODIndex, bool bInstance);

	/** Retrieves the vertex positions from the given LOD level in the Static Mesh */
	static UE_API TArray<FVector> GetVerticesForLOD(const UStaticMesh* StaticMesh, int32 LODIndex);

	/** Retrieves the vertex colors from the given LOD level in the Static Mesh */
	static UE_API TArray<FColor> GetColorDataForLOD(const UStaticMesh* StaticMesh, int32 LODIndex);

	/** Retrieves the per-instance vertex colors from the given LOD level in the StaticMeshComponent */
	static UE_API TArray<FColor> GetInstanceColorDataForLOD(const UStaticMeshComponent* MeshComponent, int32 LODIndex);

	/** Sets the specific (LOD Index) per-instance vertex colors for the given StaticMeshComponent to the supplied Color array */
	static UE_API void SetInstanceColorDataForLOD(UStaticMeshComponent* MeshComponent, int32 LODIndex, const TArray<FColor>& Colors);	
	
	/** Sets the specific (LOD Index) per-instance vertex colors for the given StaticMeshComponent to a single Color value */
	static UE_API void SetInstanceColorDataForLOD(UStaticMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor);
	
	/** Fills all vertex colors for all LODs found in the given mesh component with Fill Color */
	static UE_API void FillStaticMeshVertexColors(UStaticMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor);
	static UE_API void FillSkeletalMeshVertexColors(USkeletalMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor);
	
	/** Sets all vertex colors for a specific LOD level in the SkeletalMesh to FillColor */
	static UE_API void SetColorDataForLOD(USkeletalMesh* SkeletalMesh, int32 LODIndex, const FColor FillColor, const FColor MaskColor);

	static UE_API void ApplyFillWithMask(FColor& InOutColor, const FColor& MaskColor, const FColor& FillColor);

	/** Helper function to import Vertex Colors from a Texture to the specified MeshComponent (makes use of SImportVertexColorsOptions Widget) */
	static UE_API void ImportVertexColorsFromTexture(UMeshComponent* MeshComponent);
		
	/** Checks whether or not the given Viewport Client is a VR editor viewport client */
	static UE_API bool IsInVRMode(const FEditorViewportClient* ViewportClient);

	/** Forces the Viewport Client to render using the given Viewport Color ViewMode */
	static UE_API void SetViewportColorMode(EMeshPaintColorViewMode ColorViewMode, FEditorViewportClient* ViewportClient);

	/** Sets whether or not the level viewport should be real time rendered move or viewport as parameter? */
	static UE_API void SetRealtimeViewport(bool bRealtime);

	/** Forces the component to render LOD level at LODIndex instead of the view-based LOD level ( X = 0 means do not force the LOD, X > 0 means force the lod to X - 1 ) */
	static UE_API void ForceRenderMeshLOD(UMeshComponent* Component, int32 LODIndex);

	/** Clears all texture overrides for this component. */
	static UE_API void ClearMeshTextureOverrides(const IMeshPaintGeometryAdapter& GeometryInfo, UMeshComponent* InMeshComponent);

	/** Applies vertex color painting found on LOD 0 to all lower LODs. */
	static UE_API void ApplyVertexColorsToAllLODs(IMeshPaintGeometryAdapter& GeometryInfo, UMeshComponent* InMeshComponent);

	/** Applies the vertex colors found in LOD level 0 to all contained LOD levels in the StaticMeshComponent */
	static UE_API void ApplyVertexColorsToAllLODs(IMeshPaintGeometryAdapter& GeometryInfo, UStaticMeshComponent* StaticMeshComponent);

	/** Applies the vertex colors found in LOD level 0 to all contained LOD levels in the SkeletalMeshComponent */
	static UE_API void ApplyVertexColorsToAllLODs(IMeshPaintGeometryAdapter& GeometryInfo, USkeletalMeshComponent* SkeletalMeshComponent);

	/** Returns the number of Mesh LODs for the given MeshComponent */
	static UE_API int32 GetNumberOfLODs(const UMeshComponent* MeshComponent);

	/** OutNumLODs is set to number of Mesh LODs for the given MeshComponent and returns true, or returns false of given mesh component has no valid LODs */
	static UE_API bool TryGetNumberOfLODs(const UMeshComponent* MeshComponent, int32& OutNumLODs);
	
	/** Returns the number of Texture Coordinates for the given MeshComponent */
	static UE_API int32 GetNumberOfUVs(const UMeshComponent* MeshComponent, int32 LODIndex);

	/** Checks whether or not the mesh components contains per lod colors (for all LODs)*/
	static UE_API bool DoesMeshComponentContainPerLODColors(const UMeshComponent* MeshComponent);

	/** Retrieves the number of bytes used to store the per-instance LOD vertex color data from the static mesh component */
	static UE_API void GetInstanceColorDataInfo(const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, int32& OutTotalInstanceVertexColorBytes);

	/** Given arguments for an action, and an action - retrieves influences vertices and applies Action to them */
	static UE_API bool ApplyPerVertexPaintAction(FPerVertexPaintActionArgs& InArgs, FPerVertexPaintAction Action);

	static UE_API bool GetPerVertexPaintInfluencedVertices(FPerVertexPaintActionArgs& InArgs, TSet<int32>& InfluencedVertices);

	/** Given the adapter, settings and view-information retrieves influences triangles and applies Action to them */
	static UE_API bool ApplyPerTrianglePaintAction(IMeshPaintGeometryAdapter* Adapter, const FVector& CameraPosition, const FVector& HitPosition, const UPaintBrushSettings* Settings, FPerTrianglePaintAction Action);

	/** Applies vertex painting to InOutvertexColor according to the given parameters  */
	static UE_API bool PaintVertex(const FVector& InVertexPosition, const FMeshPaintParameters& InParams, FColor& InOutVertexColor);

	/** Applies Vertex Color Painting according to the given parameters */
	static UE_API void ApplyVertexColorPaint(const FMeshPaintParameters &InParams, const FLinearColor &OldColor, FLinearColor &NewColor, const float PaintAmount);

	/** Applies Vertex Blend Weight Painting according to the given parameters */
	static UE_API void ApplyVertexWeightPaint(const FMeshPaintParameters &InParams, const FLinearColor &OldColor, const float PaintAmount, FLinearColor &NewColor);

	/** Generate texture weight color for given number of weights and the to-paint index */
	static UE_API FLinearColor GenerateColorForTextureWeight(const int32 NumWeights, const int32 WeightIndex);

	/** Computes the Paint power multiplier value */
	static UE_API float ComputePaintMultiplier(float SquaredDistanceToVertex2D, float BrushStrength, float BrushInnerRadius, float BrushRadialFalloff, float BrushInnerDepth, float BrushDepthFallof, float VertexDepthToBrush);

	/** Checks whether or not a point is influenced by the painting brush according to the given parameters*/
	static UE_API bool IsPointInfluencedByBrush(const FVector& InPosition, const FMeshPaintParameters& InParams, float& OutSquaredDistanceToVertex2D, float& OutVertexDepthToBrush);

	static UE_API bool IsPointInfluencedByBrush(const FVector2D& BrushSpacePosition, const float BrushRadiusSquared, float& OutInRangeValue);

	template<typename T>
	static void ApplyBrushToVertex(const FVector& VertexPosition, const FMatrix& InverseBrushMatrix, const float BrushRadius, const float BrushFalloffAmount, const float BrushStrength, const T& PaintValue, T& InOutValue);

public:
	struct FPaintRay
	{
		FVector CameraLocation;
		FVector RayStart;
		FVector RayDirection;
		UViewportInteractor* ViewportInteractor;
	};

	static UE_API bool RetrieveViewportPaintRays(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI, TArray<FPaintRay>& OutPaintRays);	
protected:
	/** Imports vertex colors from a Texture to the specified Static Mesh according to user-set options */
	static UE_API void ImportVertexColorsToStaticMesh(UStaticMesh* StaticMesh, const UVertexColorImportOptions* Options, UTexture2D* Texture);

	/** Imports vertex colors from a Texture to the specified Static Mesh Component according to user-set options */
	static UE_API void ImportVertexColorsToStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent, const UVertexColorImportOptions* Options, UTexture2D* Texture);

	/** Imports vertex colors from a Texture to the specified Skeletal Mesh according to user-set options */
	static UE_API void ImportVertexColorsToSkeletalMesh(USkeletalMesh* SkeletalMesh, const UVertexColorImportOptions* Options, UTexture2D* Texture);

	/** Helper function to retrieve vertex color from a UTexture given a UVCoordinate */
	static UE_API FColor PickVertexColorFromTextureData(const uint8* MipData, const FVector2D& UVCoordinate, const UTexture2D* Texture, const FColor ColorMask);	
};

template<typename T>
void MeshPaintHelpers::ApplyBrushToVertex(const FVector& VertexPosition, const FMatrix& InverseBrushMatrix, const float BrushRadius, const float BrushFalloffAmount, const float BrushStrength, const T& PaintValue, T& InOutValue)
{
	const FVector BrushSpacePosition = InverseBrushMatrix.TransformPosition(VertexPosition);
	const FVector2D BrushSpacePosition2D(BrushSpacePosition.X, BrushSpacePosition.Y);
		
	float InfluencedValue = 0.0f;
	if (IsPointInfluencedByBrush(BrushSpacePosition2D, BrushRadius * BrushRadius, InfluencedValue))
	{
		float InnerBrushRadius = BrushFalloffAmount * BrushRadius;
		float PaintStrength = MeshPaintHelpers::ComputePaintMultiplier(BrushSpacePosition2D.SizeSquared(), BrushStrength, InnerBrushRadius, BrushRadius - InnerBrushRadius, 1.0f, 1.0f, 1.0f);

		const T OldValue = InOutValue;
		InOutValue = FMath::LerpStable(OldValue, PaintValue, PaintStrength);
	}	
}

#undef UE_API
