// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"


class IMeshPaintGeometryAdapterFactory;

/**
 * MeshPaint module interface
 */
class IMeshPaintModule : public IModuleInterface
{
public:
	virtual void RegisterGeometryAdapterFactory(TSharedRef<IMeshPaintGeometryAdapterFactory> Factory) = 0;
	virtual void UnregisterGeometryAdapterFactory(TSharedRef<IMeshPaintGeometryAdapterFactory> Factory) = 0;
};
