// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPaintTypes.h"

#include "MeshPaintSettings.generated.h"

#define UE_API MESHPAINT_API

/** Mesh paint color view modes (somewhat maps to EVertexColorViewMode engine enum.) */
UENUM()
enum class EMeshPaintColorViewMode : uint8
{
	/** Normal view mode (vertex color visualization off) */
	Normal UMETA(DisplayName = "Off"),

	/** RGB only */
	RGB UMETA(DisplayName = "RGB Channels"),

	/** Alpha only */
	Alpha UMETA(DisplayName = "Alpha Channel"),

	/** Red only */
	Red UMETA(DisplayName = "Red Channel"),

	/** Green only */
	Green UMETA(DisplayName = "Green Channel"),

	/** Blue only */
	Blue UMETA(DisplayName = "Blue Channel"),
};

UCLASS(MinimalAPI)
class UPaintBrushSettings : public UObject
{
	GENERATED_UCLASS_BODY()
public:

	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API ~UPaintBrushSettings();

	float GetBrushRadius() const { return BrushRadius; }
	UE_API void SetBrushRadius(float InRadius);

	float GetBrushStrength() const { return BrushStrength; }
	UE_API void SetBrushStrength(float InStrength);

	float GetBrushFalloff() const { return BrushFalloffAmount; }
	UE_API void SetBrushFalloff(float InFalloff);

protected:
	/** Radius of the Brush used for Painting */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Radius", UIMin = "0.01", UIMax = "2048.0", ClampMin = "0.01", ClampMax = "250000.0"))
	float BrushRadius;

public:
	/** Min and Max brush radius retrieved from config */
	float BrushRadiusMin;
	float BrushRadiusMax;

	/** Strength of the brush (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin= "0.0", ClampMax = "1.0"))
	float BrushStrength; 

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float BrushFalloffAmount;

	/** Enables "Flow" painting where paint is continually applied from the brush every tick */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Enable Brush Flow"))
	bool bEnableFlow;

	/** Whether back-facing triangles should be ignored */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Ignore Back-Facing"))
	bool bOnlyFrontFacingTriangles;
	
	/** Color view mode used to display Vertex Colors */
	UPROPERTY(EditAnywhere, Category = View)
	EMeshPaintColorViewMode ColorViewMode;
};

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UMeshPaintSettings : public UObject
{
	GENERATED_BODY()

public:

	/** Size of vertex points drawn when mesh painting is active. */
	UPROPERTY(config, EditAnywhere, Category=Visualization, Meta = (ClampMin = 0, UIMin = 0, UIMax = 10))
	float VertexPreviewSize;

};

#undef UE_API
