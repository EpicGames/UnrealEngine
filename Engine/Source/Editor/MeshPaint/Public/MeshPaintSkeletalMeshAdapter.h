// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPaintModule.h"
#include "RawIndexBuffer.h"
#include "Components/SkeletalMeshComponent.h"
#include "BaseMeshPaintGeometryAdapter.h"
#include "IMeshPaintGeometryAdapterFactory.h"

class UBodySetup;
class USkeletalMesh;
class USkeletalMeshComponent;
class UTexture;
class FSkeletalMeshRenderData;
class FSkeletalMeshLODRenderData;
class FSkeletalMeshLODModel;
class UMeshComponent;

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshes
class FMeshPaintGeometryAdapterForSkeletalMeshes : public FBaseMeshPaintGeometryAdapter
{
public:
	static void InitializeAdapterGlobals();
	static void AddReferencedObjectsGlobals(FReferenceCollector& Collector);
	static void CleanupGlobals();

	/** Start IMeshPaintGeometryAdapter Overrides */
	virtual bool Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) override;
	virtual ~FMeshPaintGeometryAdapterForSkeletalMeshes();
	virtual bool Initialize() override;
	virtual void OnAdded() override;
	virtual void OnRemoved() override;
	virtual bool IsValid() const override { return SkeletalMeshComponent && ReferencedSkeletalMesh && SkeletalMeshComponent->GetSkeletalMeshAsset() == ReferencedSkeletalMesh; }
	virtual bool SupportsTexturePaint() const override { return true; }
	virtual bool SupportsVertexPaint() const override { return SkeletalMeshComponent != nullptr; }
	virtual bool LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const override;
	virtual void QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList) override;
	virtual void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual void PreEdit() override;
	virtual void PostEdit() override;
	virtual void GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const override;
	virtual void GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance = true) const override;
	virtual void SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance = true) override;
	virtual FMatrix GetComponentToWorldMatrix() const override;
	/** End IMeshPaintGeometryAdapter Overrides */
		
	/** Start FBaseMeshPaintGeometryAdapter Overrides */
	virtual bool InitializeVertexData();
	/** End FBaseMeshPaintGeometryAdapter Overrides */
protected:
	/** Callback for when the skeletal mesh on the component is changed */
	void OnSkeletalMeshChanged();
	/** Callback for when skeletal mesh DDC data is rebuild */
	void OnPostMeshCached(USkeletalMesh* SkeletalMesh);

	/** Delegate called when skeletal mesh is changed on the component */
	FDelegateHandle SkeletalMeshChangedHandle;	

	/** Skeletal mesh component represented by this adapter */
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;
	/** Skeletal mesh currently set to the Skeletal Mesh Component */
	TObjectPtr<USkeletalMesh> ReferencedSkeletalMesh;
	/** Skeletal Mesh resource retrieved from the Skeletal Mesh */
	FSkeletalMeshRenderData* MeshResource;

	/** LOD render data (at Mesh LOD Index) containing data to change */
	FSkeletalMeshLODRenderData* LODData;
	/** LOD model (source) data (at Mesh LOD Index) containing data to change */
	FSkeletalMeshLODModel* LODModel;
	/** LOD Index for which data has to be retrieved / altered*/
	int32 MeshLODIndex;
protected:
	/** Helpers structure for keeping track of cached skeletal mesh data */
	struct FSkeletalMeshReferencers
	{
		FSkeletalMeshReferencers()
			: RestoreBodySetup(nullptr)
		{}

		struct FReferencersInfo
		{
			FReferencersInfo(USkeletalMeshComponent* InSkeletalMeshComponent, ECollisionEnabled::Type InCachedCollisionType)
				: SkeletalMeshComponent(InSkeletalMeshComponent)
				, CachedCollisionType(InCachedCollisionType)
			{}

			TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;
			ECollisionEnabled::Type CachedCollisionType;
		};

		TArray<FReferencersInfo> Referencers;
		TObjectPtr<UBodySetup> RestoreBodySetup;
	};

	typedef TMap<TObjectPtr<USkeletalMesh>, FSkeletalMeshReferencers> FMeshToComponentMap;
	static FMeshToComponentMap MeshToComponentMap;
};

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshesFactory

class FMeshPaintGeometryAdapterForSkeletalMeshesFactory : public IMeshPaintGeometryAdapterFactory
{
public:
	virtual TSharedPtr<IMeshPaintGeometryAdapter> Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) const override;
	virtual void InitializeAdapterGlobals() override { FMeshPaintGeometryAdapterForSkeletalMeshes::InitializeAdapterGlobals(); }
	virtual void AddReferencedObjectsGlobals(FReferenceCollector& Collector) override { FMeshPaintGeometryAdapterForSkeletalMeshes::AddReferencedObjectsGlobals(Collector); }
	virtual void CleanupGlobals() override { FMeshPaintGeometryAdapterForSkeletalMeshes::CleanupGlobals(); }
};
