// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPaintStaticMeshAdapter.h"

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSplineMeshes

class FMeshPaintGeometryAdapterForSplineMeshes : public FMeshPaintGeometryAdapterForStaticMeshes
{
public:
	virtual bool InitializeVertexData() override;
};

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSplineMeshesFactory

class FMeshPaintGeometryAdapterForSplineMeshesFactory : public FMeshPaintGeometryAdapterForStaticMeshesFactory
{
public:
	virtual TSharedPtr<IMeshPaintGeometryAdapter> Construct(class UMeshComponent* InComponent, int32 MeshLODIndex) const override;
};
