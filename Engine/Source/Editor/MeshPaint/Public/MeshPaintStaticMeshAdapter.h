// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMeshPaintGeometryAdapterFactory.h"
#include "BaseMeshPaintGeometryAdapter.h"
#include "RawIndexBuffer.h"
#include "Components/StaticMeshComponent.h"

class UBodySetup;
class UStaticMesh;
class UTexture;
struct FStaticMeshLODResources;

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForStaticMeshes

class FMeshPaintGeometryAdapterForStaticMeshes : public FBaseMeshPaintGeometryAdapter
{
public:
	static void InitializeAdapterGlobals();
	static void AddReferencedObjectsGlobals(FReferenceCollector& Collector);
	static void CleanupGlobals();

	/** Start IMeshPaintGeometryAdapter Overrides */
	virtual bool Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) override;
	virtual ~FMeshPaintGeometryAdapterForStaticMeshes();
	virtual bool Initialize() override;
	virtual void OnAdded() override;
	virtual void OnRemoved() override;
	virtual bool IsValid() const override { return StaticMeshComponent && StaticMeshComponent->GetStaticMesh() == ReferencedStaticMesh; }
	virtual bool SupportsTexturePaint() const override { return true; }
	virtual bool SupportsVertexPaint() const override { return StaticMeshComponent && StaticMeshComponent->CanMeshPaintVertexColors(); }
	virtual bool LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const override;	
	virtual void QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList) override;
	virtual void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;		
	virtual void GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const override;
	virtual void GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance = true) const override;
	virtual void SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance = true) override;
	virtual FMatrix GetComponentToWorldMatrix() const override;
	virtual void PreEdit() override;
	virtual void PostEdit() override;
	/** End IMeshPaintGeometryAdapter Overrides*/

	/** Begin FMeshBasePaintGeometryAdapter */
	virtual bool InitializeVertexData() override;	
	/** End FMeshBasePaintGeometryAdapter */

protected:
	/** Callback for when the static mesh data is rebuilt */
	void OnPostMeshBuild(UStaticMesh* StaticMesh);
	/** Callback for when the static mesh on the component is changed */
	void OnStaticMeshChanged(UStaticMeshComponent* StaticMeshComponent);

	/** Static mesh component represented by this adapter */
	TObjectPtr<UStaticMeshComponent> StaticMeshComponent;
	/** Static mesh currently set to the Static Mesh Component */
	TObjectPtr<UStaticMesh> ReferencedStaticMesh;
	/** LOD model (at Mesh LOD Index) containing data to change */
	FStaticMeshLODResources* LODModel;
	/** LOD Index for which data has to be retrieved / altered*/
	int32 MeshLODIndex;
protected:
	/** Helpers structure for keeping track of cached static mesh data */
	struct FStaticMeshReferencers
	{
		FStaticMeshReferencers()
			: RestoreBodySetup(nullptr)
		{}

		struct FReferencersInfo
		{
			FReferencersInfo(UStaticMeshComponent* InStaticMeshComponent, ECollisionEnabled::Type InCachedCollisionType)
				: StaticMeshComponent(InStaticMeshComponent)
				, CachedCollisionType(InCachedCollisionType)
			{}

			TObjectPtr<UStaticMeshComponent> StaticMeshComponent;
			ECollisionEnabled::Type CachedCollisionType;
		};

		TArray<FReferencersInfo> Referencers;
		TObjectPtr<UBodySetup> RestoreBodySetup;
	};

	typedef TMap<TObjectPtr<UStaticMesh>, FStaticMeshReferencers> FMeshToComponentMap;
	static FMeshToComponentMap MeshToComponentMap;
};

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForStaticMeshesFactory

class FMeshPaintGeometryAdapterForStaticMeshesFactory : public IMeshPaintGeometryAdapterFactory
{
public:
	virtual TSharedPtr<IMeshPaintGeometryAdapter> Construct(class UMeshComponent* InComponent, int32 InMeshLODIndex) const override;
	virtual void InitializeAdapterGlobals() override { FMeshPaintGeometryAdapterForStaticMeshes::InitializeAdapterGlobals(); }
	virtual void AddReferencedObjectsGlobals(FReferenceCollector& Collector) override { FMeshPaintGeometryAdapterForStaticMeshes::AddReferencedObjectsGlobals(Collector); }
	virtual void CleanupGlobals() override { FMeshPaintGeometryAdapterForStaticMeshes::CleanupGlobals(); }
};
