// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/MaterialInterface.h"
#include "Templates/Requires.h"

#include <type_traits>

#include "MeshPaintTypes.generated.h"

/** Mesh painting action (paint, erase) */
enum class EMeshPaintAction : uint8
{
	/** Paint (add color or increase blending weight) */
	Paint,

	/** Erase (remove color or decrease blending weight) */
	Erase
};

/** Mesh paint mode */
UENUM()
enum class EMeshPaintMode : uint8
{	
	/** Painting colors directly */
	PaintColors,

	/** Painting texture blend weights */
	PaintWeights
};

/** Vertex paint target */
UENUM()
enum class EMeshVertexPaintTarget : uint8
{
	/** Paint the static mesh component instance in the level */
	ComponentInstance,

	/** Paint the actual static mesh asset */
	Mesh
};

/** Mesh paint parameters */
class FMeshPaintParameters
{
public:
	EMeshPaintMode PaintMode;
	EMeshPaintAction PaintAction;
	FVector BrushPosition;
	FVector BrushNormal;
	FLinearColor BrushColor;
	float SquaredBrushRadius;
	float BrushRadialFalloffRange;
	float InnerBrushRadius;
	float BrushDepth;
	float BrushDepthFalloffRange;
	float InnerBrushDepth;
	float BrushStrength;
	FMatrix BrushToWorldMatrix;
	FMatrix InverseBrushToWorldMatrix;
	bool bWriteRed;
	bool bWriteGreen;
	bool bWriteBlue;
	bool bWriteAlpha;
	int32 TotalWeightCount;
	int32 PaintWeightIndex;
	int32 UVChannel;
};

/** Structure used to hold per-triangle data for texture painting */
struct FTexturePaintTriangleInfo
{
	FVector TriVertices[3];
	FVector2D TrianglePoints[3];
	FVector2D TriUVs[3];
};

/** Structure used to house and compare Texture and UV channel pairs */
struct FPaintableTexture
{
	UTexture*	Texture;
	int32		UVChannelIndex;

	FPaintableTexture()
		: Texture(nullptr)
		, UVChannelIndex(0)
	{}

	template <
		typename T
		UE_REQUIRES(std::is_convertible_v<T, UTexture*>)
	>
	FPaintableTexture(T InTexture, uint32 InUVChannelIndex = 0)
		: Texture(InTexture)
		, UVChannelIndex(InUVChannelIndex)
	{}

	/** Overloaded equality operator for use with TArrays Contains method. */
	bool operator==(const FPaintableTexture& rhs) const
	{
		return (Texture == rhs.Texture);
		/* && (UVChannelIndex == rhs.UVChannelIndex);*/// if we compare UVChannel we would have to duplicate the texture
	}
};

struct FPaintTexture2DData
{
	/** The original texture that we're painting */
	TObjectPtr<UTexture2D> PaintingTexture2D;
	bool bIsPaintingTexture2DModified;

	/** A copy of the original texture we're painting, used for restoration. */
	UTexture2D* PaintingTexture2DDuplicate;

	/** Render target texture for painting */
	TObjectPtr<UTextureRenderTarget2D> PaintRenderTargetTexture;

	/** Render target texture used as an input while painting that contains a clone of the original image */
	TObjectPtr<UTextureRenderTarget2D> CloneRenderTargetTexture;

	/** List of materials we are painting on */
	TArray< TObjectPtr<UMaterialInterface> > PaintingMaterials;

	/** Default ctor */
	FPaintTexture2DData() :
		PaintingTexture2D(nullptr),
		bIsPaintingTexture2DModified(false),
		PaintingTexture2DDuplicate(nullptr),
		PaintRenderTargetTexture(nullptr),
		CloneRenderTargetTexture(nullptr)
	{}

	FPaintTexture2DData(UTexture2D* InPaintingTexture2D, bool InbIsPaintingTexture2DModified = false) :
		PaintingTexture2D(InPaintingTexture2D),
		bIsPaintingTexture2DModified(InbIsPaintingTexture2DModified),
		PaintRenderTargetTexture(nullptr),
		CloneRenderTargetTexture(nullptr)
	{}

	/** Serializer */
	void AddReferencedObjects(FReferenceCollector& Collector)
	{
		// @todo MeshPaint: We're relying on GC to clean up render targets, can we free up remote memory more quickly?
		Collector.AddReferencedObject(PaintingTexture2D);
		Collector.AddReferencedObject(PaintRenderTargetTexture);
		Collector.AddReferencedObject(CloneRenderTargetTexture);
		for (int32 Index = 0; Index < PaintingMaterials.Num(); Index++)
		{
			Collector.AddReferencedObject(PaintingMaterials[Index]);
		}
	}
};

namespace MeshPaintDefs
{
	// Design constraints

	// Currently we never support more than five channels (R, G, B, A, OneMinusTotal)
	static const int32 MaxSupportedPhysicalWeights = 4;
	static const int32 MaxSupportedWeights = MaxSupportedPhysicalWeights + 1;
}

/**
*  Wrapper to expose texture targets to WPF code.
*/
struct FTextureTargetListInfo
{
	UTexture2D* TextureData;
	bool bIsSelected;
	uint32 UndoCount;
	uint32 UVChannelIndex;
	FTextureTargetListInfo(UTexture2D* InTextureData, int32 InUVChannelIndex, bool InbIsSelected = false)
		: TextureData(InTextureData)
		, bIsSelected(InbIsSelected)
		, UndoCount(0)
		, UVChannelIndex(InUVChannelIndex)
	{}
};

/**
*  Wrapper to store which of a meshes materials is selected as well as the total number of materials.
*/
struct FMeshSelectedMaterialInfo
{
	int32 NumMaterials;
	int32 SelectedMaterialIndex;

	FMeshSelectedMaterialInfo(int32 InNumMaterials)
		: NumMaterials(InNumMaterials)
		, SelectedMaterialIndex(0)
	{}
};