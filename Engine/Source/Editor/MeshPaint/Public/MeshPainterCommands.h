// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

/** Base set of mesh painter commands */
class MESHPAINT_API FMeshPainterCommands : public TCommands<FMeshPainterCommands>
{
public:
	FMeshPainterCommands();

	/**
	* Initialize commands
	*/
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> IncreaseBrushRadius;
	TSharedPtr<FUICommandInfo> DecreaseBrushRadius;

	TSharedPtr<FUICommandInfo> IncreaseBrushStrength;
	TSharedPtr<FUICommandInfo> DecreaseBrushStrength;

	TSharedPtr<FUICommandInfo> IncreaseBrushFalloff;
	TSharedPtr<FUICommandInfo> DecreaseBrushFalloff;

	TArray<TSharedPtr<FUICommandInfo>> Commands;
};