// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/GenericOctreePublic.h"
#include "Math/GenericOctree.h"

/** Triangle for use in Octree for mesh paint optimization */
struct FMeshPaintTriangle
{
	uint32 Index;
	FVector Vertices[3];
	FVector Normal;
	FBoxCenterAndExtent BoxCenterAndExtent;
};

/** Semantics for the simple mesh paint octree */
struct FMeshPaintTriangleOctreeSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	/**
	* Get the bounding box of the provided octree element. In this case, the box
	* is merely the point specified by the element.
	*
	* @param	Element	Octree element to get the bounding box for
	*
	* @return	Bounding box of the provided octree element
	*/
	FORCEINLINE static FBoxCenterAndExtent GetBoundingBox(const FMeshPaintTriangle& Element)
	{
		return Element.BoxCenterAndExtent;
	}


	/**
	* Determine if two octree elements are equal
	*
	* @param	A	First octree element to check
	* @param	B	Second octree element to check
	*
	* @return	true if both octree elements are equal, false if they are not
	*/
	FORCEINLINE static bool AreElementsEqual(const FMeshPaintTriangle& A, const FMeshPaintTriangle& B)
	{
		return (A.Index == B.Index);
	}

	/** Ignored for this implementation */
	FORCEINLINE static void SetElementId(const FMeshPaintTriangle& Element, FOctreeElementId2 Id)
	{

	}
};
typedef TOctree2<FMeshPaintTriangle, FMeshPaintTriangleOctreeSemantics> FMeshPaintTriangleOctree;